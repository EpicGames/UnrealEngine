// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Drawing;

namespace UnrealControls
{
	/// <summary>
	/// A pair of colors.
	/// </summary>
	public struct ColorPair
	{
		/// <summary>
		/// The foreground color.
		/// </summary>
		public Color? BackColor;
		/// <summary>
		/// The background color
		/// </summary>
		public Color? ForeColor;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="BackColor">The background color.</param>
		/// <param name="ForeColor">The foreground color.</param>
		public ColorPair(Color? BackColor, Color? ForeColor)
		{
			this.BackColor = BackColor;
			this.ForeColor = ForeColor;
		}
	}

}
