// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Drawing;

namespace UnrealControls
{
	/// <summary>
	/// Represents a segment of colored text within a document.
	/// </summary>
	public struct ColoredDocumentLineSegment
	{
		ColorPair mColor;
		string mText;

		/// <summary>
		/// Gets the color of the text.
		/// </summary>
		public ColorPair Color
		{
			get { return mColor; }
		}

		/// <summary>
		/// Gets the text associated with the segment.
		/// </summary>
		public string Text
		{
			get { return mText; }
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="Colors">The color of the segment.</param>
		/// <param name="Text">The text associated with the segment.</param>
		public ColoredDocumentLineSegment(ColorPair Colors, string Text)
		{
			if(Text == null)
			{
				throw new ArgumentNullException("Text");
			}

			mColor = Colors;
			mText = Text;
		}
	}
}
