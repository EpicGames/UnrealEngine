// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Drawing;

namespace UnrealControls
{
	/// <summary>
	/// Represents a range of colored characters within a line of a document.
	/// </summary>
	public class ColoredTextRange
	{
		public Color? ForeColor;
		public Color? BackColor;
		public int StartIndex;
		public int Length;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="TxtForeColor">The foreground color of the text range.</param>
		/// <param name="TxtBackColor">The background color of the text range.</param>
		/// <param name="StartIndex">The start index within the range within a line of text.</param>
		/// <param name="Length">The number of characters the range encompasses.</param>
		public ColoredTextRange(Color? TxtForeColor, Color? TxtBackColor, int StartIndex, int Length)
		{
			if(Length < 0)
			{
				throw new ArgumentOutOfRangeException("Length");
			}

			if(StartIndex < 0)
			{
				throw new ArgumentOutOfRangeException("StartIndex");
			}

			this.ForeColor = TxtForeColor;
			this.BackColor = TxtBackColor;
			this.StartIndex = StartIndex;
			this.Length = Length;
		}
	}
}
