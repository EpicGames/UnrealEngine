// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Drawing;
using System.Data;
using System.Text;
using System.Windows.Forms;
using System.Runtime.InteropServices;

namespace UnrealControls
{
    //begining is at (0,0)
    //pt1 is at (15, 15)
    //pt2 is at (20, 17)
    //this creates a box that is 20x17

    /// <summary>
    /// VS.NET style tab control.
    /// </summary>
    public partial class DynamicTabControl : UserControl
    {
        /// <summary>
        /// Collection of tab pages.
        /// </summary>
        public class DynamicTabPageCollection : IList<DynamicTabPage>, ICollection<DynamicTabPage>, IEnumerable<DynamicTabPage>
        {
            DynamicTabControl m_owner;
            List<DynamicTabPage> m_pages = new List<DynamicTabPage>();

            /// <summary>
            /// Constructor.
            /// </summary>
            /// <param name="owner">The owner of the collection.</param>
            public DynamicTabPageCollection(DynamicTabControl owner)
            {
                if(owner == null)
                {
                    throw new ArgumentNullException("owner", "Owner can not be null!");
                }

                m_owner = owner;
			}

			#region IList<DynamicTabPage> Members

			/// <summary>
            /// Returns the index of the specified item.
            /// </summary>
            /// <param name="item">The item to retrieve an index for.</param>
            /// <returns>-1 if the item is not a part of the collection.</returns>
            public int IndexOf(DynamicTabPage item)
            {
                return m_pages.IndexOf(item);
            }

            /// <summary>
            /// Inserts a new item into the collection.
            /// </summary>
            /// <param name="index">The index to insert the item.</param>
            /// <param name="item">The item to be inserted.</param>
            public void Insert(int index, DynamicTabPage item)
            {
                int curIndex = -1;
                if((curIndex = m_pages.IndexOf(item)) == -1)
                {
                    m_pages.Insert(index, item);
                    m_owner.OnTabPageAdded(index);
                }
                else
                {
                    m_owner.SelectedIndex = curIndex;
                }
            }

            /// <summary>
            /// Removes the item at the specified index.
            /// </summary>
            /// <param name="index">The index of the item to be removed.</param>
            public void RemoveAt(int index)
            {
                if(index >= 0 && index < m_pages.Count)
                {
                    DynamicTabPage page = m_pages[index];
                    m_pages.RemoveAt(index);
                    m_owner.OnTabPageRemoved(index, page);
                }
            }

            /// <summary>
            /// Overloaded [] operator.
            /// </summary>
            /// <param name="index">The index into the collection.</param>
            /// <returns>The tab page at the specified index.</returns>
            public DynamicTabPage this[int index]
            {
                get
                {
                    return m_pages[index];
                }
                set
                {
                    m_pages[index] = value;
                }
            }

            #endregion

			#region ICollection<DynamicTabPage> Members

			/// <summary>
            /// Adds a tab page to the collection.
            /// </summary>
            /// <param name="item">The tab page to be added.</param>
            public void Add(DynamicTabPage item)
            {
                //m_pages.Add(item);
                Insert(0, item);
            }

            /// <summary>
            /// Clears the collection.
            /// </summary>
            public void Clear()
            {
                m_pages.Clear();
            }

            /// <summary>
            /// Returns true if the specified item exists in the collection.
            /// </summary>
            /// <param name="item">The item to be checked.</param>
            /// <returns>True if the specified item exists in the collection.</returns>
            public bool Contains(DynamicTabPage item)
            {
                return m_pages.Contains(item);
            }

            /// <summary>
            /// Copies the collection into an array.
            /// </summary>
            /// <param name="array">The array to receive the collection.</param>
            /// <param name="arrayIndex">The index to start copying at.</param>
            public void CopyTo(DynamicTabPage[] array, int arrayIndex)
            {
                m_pages.CopyTo(array, arrayIndex);
            }

            /// <summary>
            /// The number of items in the collection.
            /// </summary>
            public int Count
            {
                get { return m_pages.Count; }
            }

            /// <summary>
            /// Returns false.
            /// </summary>
            public bool IsReadOnly
            {
                get { return false; }
            }

            /// <summary>
            /// Removes the specified item from the collection.
            /// </summary>
            /// <param name="item">The item to be removed.</param>
            /// <returns>True if the item was successfully removed.</returns>
            public bool Remove(DynamicTabPage item)
            {
                for(int i = 0; i < m_pages.Count; ++i)
                {
                    if(m_pages[i] == item)
                    {
                        RemoveAt(i);
                        return true;
                    }
                }

                return false;
            }

            #endregion

			#region IEnumerable<DynamicTabPage> Members

			/// <summary>
            /// Gets an enumerator.
            /// </summary>
            /// <returns>An enumerator.</returns>
            public IEnumerator<DynamicTabPage> GetEnumerator()
            {
                return m_pages.GetEnumerator();
            }

            #endregion

            #region IEnumerable Members

            /// <summary>
            /// Gets an enumerator.
            /// </summary>
            /// <returns>An enumerator.</returns>
            System.Collections.IEnumerator System.Collections.IEnumerable.GetEnumerator()
            {
                return m_pages.GetEnumerator();
            }

            #endregion
        }

		const int SLIDEWIDTH = 24;
		const int TABHEIGHT = 17;
		const int CURVEOFFSETY = 15;
		const int CURVEOFFSETX = 15;
		const int BOXHEIGHT = 15;
		const int BOXWIDTH = 16;
		const int ARROWLEFTOFFSET = 40;

        DynamicTabPageCollection mTabPages;
        XButton mXButton = new XButton();
        MenuArrow mMnuArrow = new MenuArrow();
        ContextMenuStrip mCtxMenu = new ContextMenuStrip();
        Font mBoldFont;
        int mSelectedTab;
        EventHandler<EventArgs> mSelectedIndexChanged;

        /// <summary>
        /// Event for when the selected tab has changed.
        /// </summary>
        public event EventHandler<EventArgs> SelectedIndexChanged
        {
            add { mSelectedIndexChanged += value; }
            remove { mSelectedIndexChanged -= value; }
        }

        /// <summary>
        /// Gets the collection of tab pages.
        /// </summary>
        public DynamicTabPageCollection TabPages
        {
            get { return mTabPages; }
        }

        /// <summary>
        /// Gets/Sets the index of the currently selected tab.
        /// </summary>
        public int SelectedIndex
        {
            get { return mSelectedTab; }
            set { SelectTab(value); }
        }

        /// <summary>
        /// Gets/Sets the currently selected tab.
        /// </summary>
        public DynamicTabPage SelectedTab
        {
            get
            {
                if(mSelectedTab >= 0 && mSelectedTab < mTabPages.Count)
                {
                    return mTabPages[mSelectedTab];
                }

                return null;
            }
            set
            {
                SelectTab(mTabPages.IndexOf(value));
            }
        }

        /// <summary>
        /// Constructor.
        /// </summary>
        public DynamicTabControl()
        {
            InitializeComponent();

            this.Controls.Add(mXButton);
            //m_xButton.Click += new XButtonClickedDelegate(m_xButton_Click);
            mXButton.Click +=new EventHandler(XButton_Click);
            this.Controls.Add(mMnuArrow);
            //m_mnuArrow.Click += new MenuArrowClickedDelegate(m_mnuArrow_Click);
            mMnuArrow.Click += new EventHandler(MnuArrow_Click);
            this.UpdateMenuItemPositions();

            mTabPages = new DynamicTabPageCollection(this);
            mBoldFont = new Font(this.Font, FontStyle.Bold);
            this.DoubleBuffered = true;
        }

        /// <summary>
        /// Called when the tab selection arrow has been clicked.
        /// </summary>
        /// <param name="sender">The tab selection arrow.</param>
        /// <param name="e">Event args.</param>
        void MnuArrow_Click(object sender, EventArgs e)
        {
            foreach(DynamicTabControlMenuItem item in mCtxMenu.Items)
            {
                item.Click -= new EventHandler(CtxMenuHandler);
            }

            mCtxMenu.Items.Clear();

            if(mTabPages.Count > 0)
            {
                for(int i = 0; i < mTabPages.Count; ++i)
                {
                    mCtxMenu.Items.Add(new DynamicTabControlMenuItem(mTabPages[i].Text, i, new EventHandler(CtxMenuHandler)));
                }

                mCtxMenu.Show(this, new Point(mMnuArrow.Bounds.Left, mMnuArrow.Bounds.Bottom));
            }
        }

        /// <summary>
        /// The context menu for the tab selection arrow.
        /// </summary>
        /// <param name="sender">The context menu</param>
        /// <param name="e">Event args.</param>
        void CtxMenuHandler(object sender, EventArgs e)
        {
            DynamicTabControlMenuItem item = sender as DynamicTabControlMenuItem;

            if(item != null)
            {
                SelectTab(item.TabPageIndex);
            }
        }

        /// <summary>
        /// Called when the X button has been clicked to close a tab.
        /// </summary>
        /// <param name="sender">The X button.</param>
        /// <param name="e">Event args.</param>
        void XButton_Click(object sender, EventArgs e)
        {
            mTabPages.RemoveAt(mSelectedTab);
        }

        /// <summary>
        /// Called to paint the tab control.
        /// </summary>
        /// <param name="e">Event args.</param>
        protected override void OnPaint(PaintEventArgs e)
        {
            StringFormat fmt = new StringFormat();
            fmt.Alignment = StringAlignment.Center;
            fmt.LineAlignment = StringAlignment.Center;
            //e.Graphics.TextRenderingHint = System.Drawing.Text.TextRenderingHint.ClearTypeGridFit;

            SizeF size;
            bool clip = true;
            int offX = 0;
            int halfSlide = SLIDEWIDTH / 2;
            bool setArrowLine = false;

            for(int i = 0; i < mTabPages.Count; ++i, offX += (int)size.Width + halfSlide)
            {
                if(i == 0 || i == mSelectedTab)
                {
                    clip = false;
                }
                else
                {
                    clip = true;
                }

                size = e.Graphics.MeasureString(mTabPages[i].Text, mBoldFont);

                if(offX + size.Width + SLIDEWIDTH >= this.Width - ARROWLEFTOFFSET && i > 0)
                {
                    setArrowLine = true;
                    break;
                }

                DrawTab(offX, size, e.Graphics, mTabPages[i].Text, fmt, clip, i == mSelectedTab, mTabPages[i]);
            }

            mMnuArrow.DrawLine = setArrowLine;
            e.Graphics.Clip = new Region();
            e.Graphics.DrawLine(SystemPens.ControlDark, offX + halfSlide, 3 + TABHEIGHT, this.Width, 3 + TABHEIGHT);

            base.OnPaint(e);
        }

        /// <summary>
        /// Called to draw a tab.
        /// </summary>
        /// <param name="offX">The X offset of the tab.</param>
        /// <param name="size">The size of the tab in pixels.</param>
        /// <param name="graphics">The GDI+ graphics object.</param>
        /// <param name="text">The text to be displayed on the tab.</param>
        /// <param name="fmt">The format of the text.</param>
        /// <param name="clip">The clipping region.</param>
        /// <param name="selectedTab">The currently selected tab.</param>
        private void DrawTab(int offX, SizeF size, Graphics graphics, string text, StringFormat fmt, bool clip, bool selectedTab, DynamicTabPage tabPage)
        {
            Rectangle tabRect = new Rectangle(offX, 3, (int)size.Width + SLIDEWIDTH, TABHEIGHT);

            Point[] points = 
            {
                new Point(tabRect.Left, tabRect.Bottom),
                new Point(tabRect.Left + CURVEOFFSETX , tabRect.Bottom - CURVEOFFSETY),
                new Point(tabRect.Left + SLIDEWIDTH, tabRect.Top),
                new Point(tabRect.Right - 2, tabRect.Top),
                new Point(tabRect.Right, tabRect.Top + 2),
                new Point(tabRect.Right, tabRect.Bottom),
            };

            if(clip)
            {
                graphics.Clip = new Region(new Rectangle(tabRect.Left + SLIDEWIDTH / 2 + 1, tabRect.Top, tabRect.Width, tabRect.Height + 1));
            }
            else
            {
                graphics.Clip = new Region();
            }

            graphics.FillPolygon(tabPage.TabBackgroundColor, points);
            graphics.DrawPolygon(SystemPens.ControlDark, points);

            RectangleF rect = new RectangleF((float)(tabRect.Left + CURVEOFFSETX), (float)tabRect.Top, (float)(tabRect.Right - tabRect.Left - CURVEOFFSETX), (float)tabRect.Bottom - (float)tabRect.Top);
            graphics.DrawString(text, selectedTab ? mBoldFont : this.Font, tabPage.TabForegroundColor, rect, fmt);

            if(selectedTab)
            {
                graphics.DrawLine(SystemPens.Window, tabRect.Left + 1, tabRect.Bottom, tabRect.Right - 1, tabRect.Bottom);
            }
        }

        /// <summary>
        /// Called when a tab page has been added.
        /// </summary>
        /// <param name="index">The index of the new tab page.</param>
        void OnTabPageAdded(int index)
        {
            mTabPages[index].TextChanged += new EventHandler(DynamicTabControl_TextChanged);
            mTabPages[index].Bounds = new Rectangle(0, TABHEIGHT + 4, this.Width, this.Height - TABHEIGHT - 4);
			this.Controls.Add(mTabPages[index]);

            if(mTabPages.Count == 1)
            {
                mSelectedTab = 0;
                //this.Controls.Add(mTabPages[0]);
                OnSelectedIndexChanged(this, new EventArgs());
                mTabPages[0].SelectTab();
            }
            else if(index <= mSelectedTab)
            {
                ++mSelectedTab;
                //OnSelectedIndexChanged(this, new EventArgs());
                SelectTab(index);
            }
            else
            {
                SelectTab(index);
            }

            this.Invalidate();
        }

        /// <summary>
        /// Called when the text of a tab page has changed.
        /// </summary>
        /// <param name="sender">The tab page.</param>
        /// <param name="e">Event args.</param>
        void DynamicTabControl_TextChanged(object sender, EventArgs e)
        {
            Invalidate();
        }

        /// <summary>
        /// Called when a mouse button has been pressed.
        /// </summary>
        /// <param name="e">Event args.</param>
        protected override void OnMouseDown(MouseEventArgs e)
        {
            if(e.Button == MouseButtons.Left && e.Y < 3 + TABHEIGHT)
            {
                int offX = 0;
                int halfSlide = SLIDEWIDTH / 2;
                SizeF size;

                using(Graphics gfx = this.CreateGraphics())
                {
                    for(int i = 0; i < mTabPages.Count; ++i, offX += (int)size.Width + halfSlide)
                    {
                        size = gfx.MeasureString(mTabPages[i].Text, mBoldFont);

                        if(offX + size.Width + SLIDEWIDTH >= this.Width - ARROWLEFTOFFSET && i > 0)
                        {
                            break;
                        }

                        Rectangle rect;
                        if(i == mSelectedTab || i == 0)
                        {
                            rect = new Rectangle(offX, 3, (int)size.Width + SLIDEWIDTH, TABHEIGHT);
                        }
                        else
                        {
                            rect = new Rectangle(offX + halfSlide, 3, (int)size.Width + halfSlide, TABHEIGHT);
                        }

                        if(rect.Contains(e.X, e.Y))
                        {
                            if(e.X >= offX + CURVEOFFSETX)
                            {
                                if(i + 1 != mSelectedTab)
                                {
                                    SelectTab(i);
                                    break;
                                }
                                else
                                {
                                    float x = offX + size.Width + halfSlide;

                                    if(PointInTriangle(new PointF(e.X, e.Y),
                                    new PointF(x + CURVEOFFSETX, 3.0f),
                                    new PointF(x, 3 + TABHEIGHT),
                                    new PointF(x + CURVEOFFSETX, 3 + TABHEIGHT)))
                                    {
                                        SelectTab(i + 1);
                                        break;
                                    }
                                    else
                                    {
                                        SelectTab(i);
                                        break;
                                    }
                                }
                            }
                            else
                            {
                                if(i == 0 || i == mSelectedTab)
                                {
                                    if(PointInTriangle(new PointF(e.X, e.Y),
                                    new PointF(offX + CURVEOFFSETX, 3.0f),
                                    new PointF(offX, 3 + TABHEIGHT),
                                    new PointF(offX + CURVEOFFSETX, 3 + TABHEIGHT)))
                                    {
                                        SelectTab(i);
                                        break;
                                    }
                                }
                                else
                                {
                                    if(PointInTriangle(new PointF(e.X, e.Y),
                                    new PointF(offX + CURVEOFFSETX, 3.0f),
                                    new PointF(offX, 3 + TABHEIGHT),
                                    new PointF(offX + CURVEOFFSETX, 3 + TABHEIGHT))
                                        && e.X > offX + halfSlide)
                                    {
                                        SelectTab(i);
                                        break;
                                    }
                                }
                            } //end else
                        } //end if pt is in tab rect
                    } //end for
                } //end using
            } //end if

            base.OnMouseDown(e);
        }

        /// <summary>
        /// Returns true if the specified point is within the specified triangle.
        /// </summary>
        /// <param name="i">The point in question.</param>
        /// <param name="p0">Triangle p1.</param>
        /// <param name="p1">Triangle p2.</param>
        /// <param name="p2">Triangle p3.</param>
        /// <returns>True if i is within the triangle defined by p1, p2, and p3.</returns>
        private bool PointInTriangle(PointF i, PointF p0, PointF p1, PointF p2)
        {
            PointF e0 = new PointF(i.X - p0.X, i.Y - p0.Y);
            PointF e1 = new PointF(p1.X - p0.X, p1.Y - p0.Y);
            PointF e2 = new PointF(p2.X - p0.X, p2.Y - p0.Y);

            float d = e2.Y * e1.X - e2.X * e1.Y;

            if(d == 0.0f)
            {
                return false;
            }

            float u = (e0.Y * e1.X - e0.X * e1.Y) / d;

            if(u < 0.0f || u > 1.0f)
            {
                return false;
            }

            float v = (e0.X - e2.X * u) / e1.X;

            if(v < 0.0f)
            {
                return false;
            }

            if(v + u > 1.0f)
            {
                return false;
            }

            return true;
        }

        /// <summary>
        /// Selects the tab at the specified index.
        /// </summary>
        /// <param name="i">The index of the tab to be selected.</param>
        private void SelectTab(int i)
        {
            if(mSelectedTab != i && i >= 0 && i < mTabPages.Count)
            {
                if(mSelectedTab >= 0 && mSelectedTab < mTabPages.Count)
                {
                    mTabPages[mSelectedTab].DeselectTab();
                    //this.Controls.Remove(mTabPages[mSelectedTab]);
					mTabPages[mSelectedTab].Visible = false;
                }

                if(TabInView(i))
                {
                    mSelectedTab = i;
                    //this.Controls.Add(mTabPages[i]);
					mTabPages[i].Visible = true;
                    mTabPages[i].SelectTab();
                }
                else
                {
                    DynamicTabPage tab = mTabPages[i];
                    mTabPages.RemoveAt(i);
                    mTabPages.Insert(0, tab);
                    mSelectedTab = 0;
                    //this.Controls.Add(mTabPages[0]);
					mTabPages[0].Visible = true;
                    mTabPages[0].SelectTab();
                }

                OnSelectedIndexChanged(this, new EventArgs());

                Invalidate();
            }
        }

        /// <summary>
        /// Returns true if the tab at the specified index is in view.
        /// </summary>
        /// <param name="index">The index of the tab to be checked.</param>
        /// <returns>True if the tab with the specified index is in view.</returns>
        public bool TabInView(int index)
        {
            int offX = 0;
            int halfSlide = SLIDEWIDTH / 2;
            SizeF size;

            using(Graphics gfx = this.CreateGraphics())
            {
                for(int i = 0; i < mTabPages.Count; ++i, offX += (int)size.Width + halfSlide)
                {
                    size = gfx.MeasureString(mTabPages[i].Text, mBoldFont);

                    if(offX + size.Width + SLIDEWIDTH >= this.Width - ARROWLEFTOFFSET && i > 0)
                    {
                        break;
                    }

                    if(i == index)
                    {
                        return true;
                    }
                }
            }

            return false;
        }

        /// <summary>
        /// Called when a control has been added to the tab page.
        /// </summary>
        /// <param name="e"></param>
        protected override void OnControlAdded(ControlEventArgs e)
        {
            if(!(e.Control is DynamicTabPage || e.Control is XButton || e.Control is MenuArrow))
            {
                this.Controls.Remove(e.Control);
            }
        }

        /// <summary>
        /// Called when the tab control has been resized.
        /// </summary>
        /// <param name="e">Event args.</param>
        protected override void OnResize(EventArgs e)
        {
            base.OnResize(e);
            UpdateMenuItemPositions();

            if(mTabPages != null)
            {
                foreach(DynamicTabPage page in mTabPages)
                {
                    page.Bounds = new Rectangle(0, TABHEIGHT + 4, this.Width, this.Height - TABHEIGHT - 4);
                }
            }

			Invalidate(false);
        }

        /// <summary>
        /// Updates the arrow button and X button positions.
        /// </summary>
        void UpdateMenuItemPositions()
        {
            mMnuArrow.Bounds = new Rectangle(this.Width - ARROWLEFTOFFSET, 3, BOXWIDTH, BOXHEIGHT);
			mXButton.Bounds = new Rectangle((this.Width - ARROWLEFTOFFSET) + BOXWIDTH + 3, 3, BOXWIDTH, BOXHEIGHT);
        }

        /// <summary>
        /// Called when a tab page has been removed.
        /// </summary>
        /// <param name="index">The index of the removed tab page.</param>
        /// <param name="page">The tab page being removed.</param>
        internal void OnTabPageRemoved(int index, DynamicTabPage page)
        {
            page.TextChanged -= new EventHandler(this.DynamicTabControl_TextChanged);

            if(index == mSelectedTab)
            {
                page.DeselectTab();
                this.Controls.Remove(page);

                if(mSelectedTab < mTabPages.Count)
                {
                    //this.Controls.Add(mTabPages[mSelectedTab]);
					mTabPages[mSelectedTab].Visible = true;
                    mTabPages[mSelectedTab].SelectTab();
                }
                else
                {
                    mSelectedTab = mTabPages.Count - 1;

                    if(mSelectedTab >= 0)
                    {
                        //this.Controls.Add(mTabPages[mSelectedTab]);
						mTabPages[mSelectedTab].Visible = true;
                        mTabPages[mSelectedTab].SelectTab();
                    }
                }

                OnSelectedIndexChanged(this, new EventArgs());
            }
            else if(index < mSelectedTab)
            {
                --mSelectedTab;
                OnSelectedIndexChanged(this, new EventArgs());
            }

            Invalidate();
        }

        /// <summary>
        /// Called when the index of the selected tab page has changed.
        /// </summary>
        /// <param name="sender">The tab.</param>
        /// <param name="e">Event args.</param>
        protected virtual void OnSelectedIndexChanged(object sender, EventArgs e)
        {
            if(mSelectedIndexChanged != null)
            {
                mSelectedIndexChanged(sender, e);
            }
        }
    }
}
