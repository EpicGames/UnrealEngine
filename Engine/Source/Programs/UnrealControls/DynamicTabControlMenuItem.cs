// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Windows.Forms;
using System.Drawing;

namespace UnrealControls
{
    /// <summary>
    /// A tab control menu item.
    /// </summary>
    public class DynamicTabControlMenuItem : ToolStripMenuItem
    {
        int m_index;

        /// <summary>
        /// The index of the tab control.
        /// </summary>
        public int TabPageIndex
        {
            get { return m_index; }
        }

        /// <summary>
        /// Constructor.
        /// </summary>
        /// <param name="text">The name of the tab.</param>
        /// <param name="index">The index of the tab.</param>
        public DynamicTabControlMenuItem(string text, int index)
        {
            this.Text = text;
            this.Name = text;
            m_index = index;
        }

        /// <summary>
        /// Constructor.
        /// </summary>
        /// <param name="text">The name of the tab.</param>
        /// <param name="index">The index of the tab.</param>
        /// <param name="handler">The event handler for the menu item.</param>
        public DynamicTabControlMenuItem(string text, int index, EventHandler handler)
        {
            this.Text = text;
            this.Name = text;
            this.Click += handler;
            m_index = index;
        }
    }
}
