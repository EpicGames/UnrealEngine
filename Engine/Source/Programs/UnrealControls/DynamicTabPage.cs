// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Windows.Forms;
using System.Drawing;
using System.ComponentModel;

namespace UnrealControls
{
    /// <summary>
    /// A VS.NET style tab page.
    /// </summary>
    public class DynamicTabPage : UserControl
    {
		Brush mTabForeColorBrush = SystemBrushes.ControlText;
		Brush mTabBackgroundColorBrush = SystemBrushes.Window;
		EventHandler<EventArgs> mTabSelected;
		EventHandler<EventArgs> mTabDeselected;

        /// <summary>
        /// Event for when the tab has been selected.
        /// </summary>
		public event EventHandler<EventArgs> TabSelected
        {
            add { mTabSelected += value; }
            remove { mTabSelected -= value; }
        }

        /// <summary>
        /// Event for when the tab has been deselected.
        /// </summary>
		public event EventHandler<EventArgs> TabDeselected
        {
            add { mTabDeselected += value; }
            remove { mTabDeselected -= value; }
        }

		/// <summary>
		/// Gets/Sets the foreground color for this tab pages tab.
		/// </summary>
		public Brush TabForegroundColor
		{
			get { return mTabForeColorBrush; }
			set
			{
				if(value != null && value != mTabForeColorBrush)
				{
					mTabForeColorBrush = value;

					if(Parent != null)
					{
						Parent.Invalidate();
					}
				}
			}
		}

		/// <summary>
		/// Gets/Sets the background color for this tab pages tab.
		/// </summary>
		public Brush TabBackgroundColor
		{
			get { return mTabBackgroundColorBrush; }
			set
			{
				if(value != null && value != mTabBackgroundColorBrush)
				{
					mTabBackgroundColorBrush = value;

					if(Parent != null)
					{
						Parent.Invalidate();
					}
				}
			}
		}

        /// <summary>
        /// The dock style of the tab.
        /// </summary>
        [Browsable(false)]
        public override DockStyle Dock
        {
            get
            {
                return base.Dock;
            }
            set
            {
                base.Dock = value;
            }
        }

        /// <summary>
        /// Constructor.
        /// </summary>
        public DynamicTabPage()
        {
            this.BackColor = SystemColors.Window;
        }

        /// <summary>
        /// Constructor.
        /// </summary>
        /// <param name="text">Name of the tab.</param>
        public DynamicTabPage(string text)
        {
            this.BackColor = SystemColors.Window;
            this.Text = text;
        }

        /// <summary>
        /// Called when the tab has been selected.
        /// </summary>
        /// <param name="e">Event args.</param>
        protected virtual void OnTabSelected(EventArgs e)
        {
            if(mTabSelected != null)
            {
                mTabSelected(this, e);
            }
        }

        /// <summary>
        /// Called when the tab has been deselected.
        /// </summary>
        /// <param name="e"></param>
        protected virtual void OnTabDeselected(EventArgs e)
        {
            if(mTabDeselected != null)
            {
                mTabDeselected(this, e);
            }
        }

        /// <summary>
        /// Called to select a tab.
        /// </summary>
        internal void SelectTab()
        {
            OnTabSelected(new EventArgs());
        }

        /// <summary>
        /// Deselects the tab.
        /// </summary>
        internal void DeselectTab()
        {
            OnTabDeselected(new EventArgs());
        }
    }
}
