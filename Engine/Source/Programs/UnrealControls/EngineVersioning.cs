// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections.Generic;
using System.Globalization;
using System.IO;
using System.Reflection;

namespace UnrealControls
{
	public class EngineVersioning
	{
		public DateTime TimeStamp = DateTime.MinValue;
		public int EngineVersion = -1;
		public int Changelist = -1;
		public List<string> SpecialDefines = new List<string>();

		public EngineVersioning()
		{
			// Get the owning assembly that loaded up this instance of UnrealControls
			Assembly EntryAssembly = Assembly.GetEntryAssembly();

			string PropertiesFilePath = Path.Combine( Path.GetDirectoryName( EntryAssembly.Location ), "build.properties" );
			FileInfo Info = new FileInfo( PropertiesFilePath );
			if( Info.Exists )
			{
				StreamReader Reader = Info.OpenText();

				string Line = Reader.ReadLine();
				while( Line != null )
				{
					if( Line.ToLower().StartsWith( "timestampforbvt=" ) )
					{
						try
						{
							string CleanDateTime = Line.Substring( "timestampforbvt=".Length );
							TimeStamp = DateTime.ParseExact( CleanDateTime, "yyyy-MM-dd_HH.mm.ss", CultureInfo.InvariantCulture );
						}
						catch
						{
						}
					}
					else if( Line.ToLower().StartsWith( "changelistbuiltfrom=" ) )
					{
						try
						{
							Changelist = Int32.Parse( Line.Substring( "changelistbuiltfrom=".Length ) );
						}
						catch
						{
						}
					}
					else if( Line.ToLower().StartsWith( "engineversion=" ) )
					{
						try
						{
							EngineVersion = Int32.Parse( Line.Substring( "engineversion=".Length ) );
						}
						catch
						{
						}
					}
					else
					{
						SpecialDefines.Add( Line );
					}

					Line = Reader.ReadLine();
				}

				Reader.Close();
			}
		}
	}
}
