// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;

namespace UnrealControls
{
	public struct FindResult
	{
		public static readonly FindResult Empty = new FindResult(-1, -1, 0);

		public int Line;
		public int Column;
		public int Length;

		public FindResult(int Line, int Column, int Length)
		{
			this.Line = Line;
			this.Column = Column;
			this.Length = Length;
		}

		public override bool Equals(object obj)
		{
			if(obj is FindResult)
			{
				return this == (FindResult)obj;
			}

			return base.Equals(obj);
		}

		public override int GetHashCode()
		{
			return base.GetHashCode();
		}

		public static bool operator ==(FindResult Left, FindResult Right)
		{
			return Left.Column == Right.Column && Left.Length == Right.Length && Left.Line == Right.Line;
		}

		public static bool operator !=(FindResult Left, FindResult Right)
		{
			return Left.Column != Right.Column || Left.Length != Right.Length || Left.Line != Right.Line;
		}
	}
}
