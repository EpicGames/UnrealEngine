// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Reflection;
using System.Text;
using System.Xml;
using System.Xml.Serialization;

namespace UnrealControls
{
	static public class GameLocator
	{
		static public List<string> LocateGames( string BranchRoot )
		{
			// Create a container to list the valid games
			List<string> ValidGames = new List<string>();

			// Find all potential game folders
			DirectoryInfo DirInfo = new DirectoryInfo( BranchRoot );
			if( DirInfo.Exists )
			{
				// Iterate over the potentials
				foreach( DirectoryInfo SubDirInfo in DirInfo.GetDirectories( "*Game" ) )
				{
					// Check for an uncooked game
					DirectoryInfo[] ConfigDirInfo = SubDirInfo.GetDirectories( "config" );
					DirectoryInfo[] ContentDirInfo = SubDirInfo.GetDirectories( "content" );
					DirectoryInfo[] ScriptDirInfo = SubDirInfo.GetDirectories( "script" );

					if( ConfigDirInfo.Length > 0 && ContentDirInfo.Length > 0 && ScriptDirInfo.Length > 0 )
					{
						string GameName = SubDirInfo.Name.Substring( 0, SubDirInfo.Name.Length - "Game".Length );
						ValidGames.Add( GameName );
					}
					else
					{
						// Check for a cooked game
						DirectoryInfo[] CookedDirInfo = SubDirInfo.GetDirectories( "cooked*" );
						FileInfo[] TocInfo = SubDirInfo.GetFiles( "*.txt" );

						if( CookedDirInfo.Length > 0 )
						{
							if( TocInfo.Length > 0 )
							{
								string GameName = SubDirInfo.Name.Substring( 0, SubDirInfo.Name.Length - "Game".Length );
								ValidGames.Add( GameName );
							}
						}
					}
				}
			}

			return ( ValidGames );
		}

		static public List<string> LocateGames()
		{
			// Get the owning assembly that loaded up this instance of UnrealControls
			Assembly EntryAssembly = Assembly.GetEntryAssembly();

			// Find the root of the branch
			string BranchRoot = Path.Combine( Path.GetDirectoryName( EntryAssembly.Location ), ".." );

			return ( LocateGames( BranchRoot ) );
		}
	}

	static public class LanguageLocator
	{
		static public List<string> LocateLanguages( string Game )
		{
			// Create a container to list the valid languages
			List<string> ValidLanguages = new List<string>();

			// Find all potential game folders
			string GameLocFolder = "..\\" + Game + "Game\\Localization";
			DirectoryInfo DirInfo = new DirectoryInfo( GameLocFolder );

			foreach( DirectoryInfo SubDirInfo in DirInfo.GetDirectories( "???" ) )
			{
				ValidLanguages.Add( SubDirInfo.Name.ToUpper() );
			}

			return ( ValidLanguages );
		}
	}

	static public class DefaultTargetDirectory
	{
		static public string GetDefaultTargetDirectory()
		{
			// Get the owning assembly that loaded up this instance of UnrealControls
			Assembly EntryAssembly = Assembly.GetEntryAssembly();

			// Find the root of the branch
			string BranchRoot = Path.Combine( Path.GetDirectoryName( EntryAssembly.Location ), ".." );
			DirectoryInfo BranchRootInfo = new DirectoryInfo( BranchRoot );

			// Set to name of the local branch
			string DefaultTargetDirectory = BranchRootInfo.Name;

			// Replace 'unrealengine3' with 'UE3'
			int UnrealEngine3Index = DefaultTargetDirectory.ToLower().IndexOf( "unrealengine3" );
			if( UnrealEngine3Index >= 0 )
			{
				DefaultTargetDirectory = DefaultTargetDirectory.Remove( UnrealEngine3Index, "unrealengine3".Length );
				DefaultTargetDirectory = DefaultTargetDirectory.Insert( UnrealEngine3Index, "UE3" );
			}

			// Add in the machine name
			if(System.Net.Dns.GetHostName().Length > 0 )
			{
				DefaultTargetDirectory += "-" + System.Net.Dns.GetHostName();
			}

			return ( DefaultTargetDirectory );
		}
	}

	static public class XmlHandler
	{
		static private void XmlSerializer_UnknownAttribute( object sender, XmlAttributeEventArgs e )
		{
		}

		static private void XmlSerializer_UnknownNode( object sender, XmlNodeEventArgs e )
		{
		}

		static public T ReadXml<T>( string FileName ) where T : new()
		{
			T Instance = new T();
			StreamReader XmlStream = null;
			try
			{
				// Get the XML data stream to read from
				XmlStream = new StreamReader( FileName );

				// Creates an instance of the XmlSerializer class so we can read the settings object
				XmlSerializer ObjSer = new XmlSerializer( typeof( T ) );
				// Add our callbacks for a busted XML file
				ObjSer.UnknownNode += new XmlNodeEventHandler( XmlSerializer_UnknownNode );
				ObjSer.UnknownAttribute += new XmlAttributeEventHandler( XmlSerializer_UnknownAttribute );

				// Create an object graph from the XML data
				Instance = ( T )ObjSer.Deserialize( XmlStream );
			}
			catch( Exception E )
			{
				Debug.WriteLine( E.Message );
			}
			finally
			{
				if( XmlStream != null )
				{
					// Done with the file so close it
					XmlStream.Close();
				}
			}

			return ( Instance );
		}

		static public bool WriteXml<T>( T Data, string FileName, string DefaultNameSpace )
		{
			bool bSuccess = true;
			StreamWriter XmlStream = null;
			try
			{
				FileInfo Info = new FileInfo( FileName );
				if( Info.Exists )
				{
					Info.IsReadOnly = false;
				}

				XmlSerializerNamespaces EmptyNameSpace = new XmlSerializerNamespaces();
				EmptyNameSpace.Add( "", DefaultNameSpace );

				XmlStream = new StreamWriter( FileName, false, Encoding.Unicode );
				XmlSerializer ObjSer = new XmlSerializer( typeof( T ) );

				// Add our callbacks for a busted XML file
				ObjSer.UnknownNode += new XmlNodeEventHandler( XmlSerializer_UnknownNode );
				ObjSer.UnknownAttribute += new XmlAttributeEventHandler( XmlSerializer_UnknownAttribute );

				ObjSer.Serialize( XmlStream, Data, EmptyNameSpace );
			}
			catch( Exception E )
			{
				Debug.WriteLine( E.Message );
				bSuccess = false;
			}
			finally
			{
				if( XmlStream != null )
				{
					// We're finished with the file so close it
					XmlStream.Close();
				}
			}

			return ( bSuccess );
		}
	}

	static public class ProcessHandler
	{
		// public delegate void CaptureOutputDelegate( object Sender, DataReceivedEventArgs e );
		public delegate void CaptureMessageDelegate( string Message );
		static CaptureMessageDelegate CaptureOutput = null;

		static public void CaptureOutputCallback( object Sender, DataReceivedEventArgs e )
		{
			CaptureOutput( e.Data );
		}

		static public int SpawnProcessAndWait( string Executable, string CWD, CaptureMessageDelegate CaptureOutputInstance, params string[] Parameters )
		{
			Process Utility = new Process();
			try
			{
				FileInfo Info = new FileInfo( Executable );
				if( !Info.Exists )
				{
					return ( -1 );
				}

				Utility.StartInfo.FileName = Executable;
				foreach( string Parameter in Parameters )
				{
					Utility.StartInfo.Arguments += Parameter + " ";
				}
				Utility.StartInfo.WorkingDirectory = CWD;

				Utility.StartInfo.CreateNoWindow = true;
				Utility.StartInfo.UseShellExecute = false;

				if( CaptureOutputInstance != null )
				{
					CaptureOutput = CaptureOutputInstance;
					Utility.StartInfo.RedirectStandardOutput = true;
					Utility.StartInfo.RedirectStandardError = true;
					Utility.OutputDataReceived += new DataReceivedEventHandler( CaptureOutputCallback );
					Utility.ErrorDataReceived += new DataReceivedEventHandler( CaptureOutputCallback );

					CaptureOutput( "Launching: " + Executable + " " + Utility.StartInfo.Arguments + "(CWD: " + Utility.StartInfo.WorkingDirectory + ")" );
				}

				if( !Utility.Start() )
				{
					return ( -2 );
				}

				if( CaptureOutputInstance != null )
				{
					Utility.BeginOutputReadLine();
					Utility.BeginErrorReadLine();
					Utility.EnableRaisingEvents = true;
				}

				while( !Utility.HasExited )
				{
					Utility.WaitForExit( 50 );
				}
			}
			catch( Exception )
			{
				return ( -3 );
			}

			return ( Utility.ExitCode );
		}
	}
}
