// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Drawing;
using System.Data;
using System.Text;
using System.Windows.Forms;
using System.Windows.Forms.VisualStyles;
using System.Drawing.Drawing2D;

namespace UnrealControls
{
    //public delegate void MenuArrowClickedDelegate(object sender, EventArgs e);

    /// <summary>
    /// The arrow button on a VS.NET style tab control.
    /// </summary>
    public partial class MenuArrow : UserControl
    {
		ProfessionalColorTable mColorTable = new ProfessionalColorTable();
        bool mDrawLine = false;
        bool mMouseOver = false;

        /// <summary>
        /// Gets/Sets whether or not to draw a line above the arrow.
        /// </summary>
        public bool DrawLine
        {
            get { return mDrawLine; }
            set 
            {
                if(mDrawLine != value)
                {
                    mDrawLine = value;
                    Invalidate();
                }
            }
        }

        /// <summary>
        /// Gets whether or not the mouse is currently over the control.
        /// </summary>
        public bool MouseOver
        {
            get { return mMouseOver; }
        }

        /// <summary>
        /// Constructor.
        /// </summary>
        public MenuArrow()
        {
            InitializeComponent();
        }

        /// <summary>
        /// Called to draw the control.
        /// </summary>
        /// <param name="e">Event args.</param>
        protected override void OnPaint(PaintEventArgs e)
        {
            Point[] arrowPts = 
                {
                    new Point(3, this.Height - 8),
                    new Point(this.Width / 2, this.Height - 3),
                    new Point(this.Width - 3, this.Height - 8),
                };

            if(mMouseOver)
            {
				Rectangle rect = new Rectangle(0, 0, this.Width - 1, this.Height - 1);

				using(LinearGradientBrush bgBrush = new LinearGradientBrush(rect, mColorTable.MenuItemSelectedGradientBegin, mColorTable.MenuItemSelectedGradientEnd, LinearGradientMode.Vertical))
				{
					e.Graphics.FillRectangle(bgBrush, rect);
				}

				using(Pen pen = new Pen(mColorTable.MenuItemBorder))
				{
					e.Graphics.DrawRectangle(pen, rect);
				}
            }

            e.Graphics.FillPolygon(SystemBrushes.ControlText, arrowPts);

            if(mDrawLine)
            {
                e.Graphics.FillRectangle(SystemBrushes.ControlText, 3, 3, this.Width - 6, 2);
            }

            base.OnPaint(e);
        }

        /// <summary>
        /// Called when the mouse enters the control.
        /// </summary>
        /// <param name="e">Event args.</param>
        protected override void OnMouseEnter(EventArgs e)
        {
            mMouseOver = true;
            base.OnMouseEnter(e);

            Invalidate();
        }

        /// <summary>
        /// Called when the mouse leaves the control.
        /// </summary>
        /// <param name="e">Event args.</param>
        protected override void OnMouseLeave(EventArgs e)
        {
            mMouseOver = false;
            base.OnMouseLeave(e);

            Invalidate();
        }

        //protected override void OnMouseClick(MouseEventArgs e)
        //{
        //    base.OnMouseClick(e);

        //    if(m_mnuClickedHandlers != null)
        //    {
        //        m_mnuClickedHandlers(this, new EventArgs());
        //    }
        //}
    }
}
