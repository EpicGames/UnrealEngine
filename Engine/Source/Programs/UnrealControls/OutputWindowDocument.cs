// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Windows.Forms;
using System.IO;
using System.Drawing;

namespace UnrealControls
{
	/// <summary>
	/// This delegate is for creating filtered documents.
	/// </summary>
	/// <param name="Line">The line of text that is being checked for filtering.</param>
	/// <param name="Data">User supplied data that may control filtering.</param>
	/// <returns>True if the line of text is to be filtered out of the document.</returns>
	public delegate bool OutputWindowDocumentFilterDelegate(OutputWindowDocument.ReadOnlyDocumentLine Line, object Data);

	/// <summary>
	/// A document that can be viewed by multiple <see cref="OutputWindowView"/>'s.
	/// </summary>
	public class OutputWindowDocument
	{
		/// <summary>
		/// Represents a line of colored text within a document.
		/// </summary>
		internal class DocumentLine : ICloneable
		{
			//StringBuilder mText = new StringBuilder();
			StringBuilder mBldrText = new StringBuilder();
			string mFinalText;
			List<ColoredTextRange> mColorRanges = new List<ColoredTextRange>();
			ICloneable mUserData;

			/// <summary>
			/// Gets the length of the line in characters.
			/// </summary>
			public int Length
			{
				get { return mFinalText == null ? mBldrText.Length : mFinalText.Length; }
			}

			/// <summary>
			/// Gets the character at the specified index.
			/// </summary>
			/// <param name="Index">The index of the character to get.</param>
			/// <returns>The character at the specified index.</returns>
			public char this[int Index]
			{
				get { return mFinalText == null ? mBldrText[Index] : mFinalText[Index]; }
			}

			/// <summary>
			/// Gets/Sets the user data associated with the line.
			/// </summary>
			public ICloneable UserData
			{
				get { return mUserData; }
				set { mUserData = value; }
			}

			/// <summary>
			/// Appends a character to the line of text.
			/// </summary>
			/// <param name="TxtColor">The color of the character.</param>
			/// <param name="CharToAppend">The character to append.</param>
			public void Append(Color? TxtColor, char CharToAppend)
			{
				System.Diagnostics.Debug.Assert(mBldrText != null);

				if(mColorRanges.Count == 0)
				{
					mColorRanges.Add(new ColoredTextRange(TxtColor, null, 0, 1));
				}
				else
				{
					int CurColorIndex = mColorRanges.Count - 1;

					if(mColorRanges[CurColorIndex].ForeColor != TxtColor)
					{
						mColorRanges.Add(new ColoredTextRange(TxtColor, null, mBldrText.Length, 1));
					}
					else
					{
						++mColorRanges[CurColorIndex].Length;
					}
				}

				mBldrText.Append(CharToAppend);

				System.Diagnostics.Debug.Assert(CountEOL() <= 1);
			}

			/// <summary>
			/// This is a debug function that counts the number of EOL's in a line. There should only ever be 1.
			/// </summary>
			/// <returns>The number of EOL's in the line.</returns>
			int CountEOL()
			{
				System.Diagnostics.Debug.Assert(mBldrText != null);

				int NumEOL = 0;

				for(int i = 0; i < mBldrText.Length; ++i)
				{
					if(mBldrText[i] == '\n' && i > 0 && mBldrText[i - 1] == '\r')
					{
						++NumEOL;
					}
				}

				return NumEOL;
			}

			/// <summary>
			/// Appends text to the end of the line.
			/// </summary>
			/// <param name="TxtColor">The color of the text.</param>
			/// <param name="TxtToAppend">The text to append.</param>
			public void Append(Color? TxtColor, string TxtToAppend)
			{
				System.Diagnostics.Debug.Assert(mBldrText != null);

				if(mColorRanges.Count == 0)
				{
					mColorRanges.Add(new ColoredTextRange(TxtColor, null, 0, TxtToAppend.Length));
				}
				else
				{
					int CurColorIndex = mColorRanges.Count - 1;

					if(mColorRanges[CurColorIndex].ForeColor != TxtColor)
					{
						mColorRanges.Add(new ColoredTextRange(TxtColor, null, mBldrText.Length, TxtToAppend.Length));
					}
					else
					{
						mColorRanges[CurColorIndex].Length += TxtToAppend.Length;
					}
				}

				mBldrText.Append(TxtToAppend);

				System.Diagnostics.Debug.Assert(CountEOL() <= 1);
			}

			/// <summary>
			/// Gets an array of colored line segments that can be used for drawing.
			/// </summary>
			/// <returns>An array of colored line segments.</returns>
			public ColoredDocumentLineSegment[] GetLineSegments()
			{
				ColoredDocumentLineSegment[] Segments = new ColoredDocumentLineSegment[mColorRanges.Count];
				int CurSegment = 0;

				foreach(ColoredTextRange CurRange in mColorRanges)
				{
					Segments[CurSegment] = new ColoredDocumentLineSegment(new ColorPair(CurRange.BackColor, CurRange.ForeColor), this.ToString(CurRange.StartIndex, CurRange.Length));
				}

				return Segments;
			}

			/// <summary>
			/// Gets an array of colored line segments that can be used for drawing.
			/// </summary>
			/// <param name="StartIndex">The starting character at which to begin generating segments.</param>
			/// <param name="Length">The number of characters to include in segment generation.</param>
			/// <returns>An array of colored line segments.</returns>
			public ColoredDocumentLineSegment[] GetLineSegments(int StartIndex, int Length)
			{
				List<ColoredDocumentLineSegment> Segments = new List<ColoredDocumentLineSegment>(mColorRanges.Count);
				int CharsToCopy = 0;
				bool bFoundStartSegment = false;

				foreach(ColoredTextRange CurRange in mColorRanges)
				{
					if(bFoundStartSegment)
					{
						CharsToCopy = Math.Min(Length, CurRange.Length);

						Segments.Add(new ColoredDocumentLineSegment(new ColorPair(CurRange.BackColor, CurRange.ForeColor), this.ToString(CurRange.StartIndex, CharsToCopy)));

						Length -= CharsToCopy;

						if(Length <= 0)
						{
							break;
						}
					}
					else
					{
						if(CurRange.StartIndex <= StartIndex)
						{
							bFoundStartSegment = true;
							CharsToCopy = Math.Min(Length, CurRange.Length);

							Segments.Add(new ColoredDocumentLineSegment(new ColorPair(CurRange.BackColor, CurRange.ForeColor), this.ToString(Math.Max(StartIndex, CurRange.StartIndex), CharsToCopy)));

							Length -= CharsToCopy;

							if(Length <= 0)
							{
								break;
							}
						}
					}
				}

				return Segments.ToArray();
			}

			/// <summary>
			/// Gets an array of colored line segments that can be used for drawing.
			/// </summary>
			/// <remarks>
			/// If the line of text contains any instances of <paramref name="FindText"/> they will be replaced with new line segments that contain <paramref name="InsertionColors"/> for drawing.
			/// </remarks>
			/// <param name="StartIndex">The starting character at which to begin generating segments.</param>
			/// <param name="Length">The number of characters to include in segment generation.</param>
			/// <param name="FindText">Text to search for that requires special coloring.</param>
			/// <param name="InsertionColors">The colors to use when drawing <paramref name="FindText"/>.</param>
			/// <param name="ComparisonType">The type of string comparison that will be conducted when searching for <paramref name="FindText"/>.</param>
			/// <param name="bHasFindText">Set to True if <paramref name="FindText"/> exists within the line segments.</param>
			/// <returns>An array of colored line segments.</returns>
			public ColoredDocumentLineSegment[] GetLineSegmentsWithFindString(int StartIndex, int Length, string FindText, ColorPair InsertionColors, StringComparison ComparisonType, out bool bHasFindText)
			{
				string LineText = this.ToString();
				int FindTextIndex = LineText.IndexOf(FindText, ComparisonType);
				int EndLineIndex = StartIndex + Length;
				List<ColoredTextRange> FindTextSegments = new List<ColoredTextRange>(mColorRanges.Count);
				List<ColoredDocumentLineSegment> Segments = new List<ColoredDocumentLineSegment>(mColorRanges.Count * 2);

				while(FindTextIndex != -1)
				{
					if(FindTextIndex >= EndLineIndex)
					{
						break;
					}

					// If any part of the text is past StartIndex it will be visible so add it to the array
					if(FindTextIndex + FindText.Length > StartIndex)
					{
						if(FindTextIndex >= StartIndex)
						{
							FindTextSegments.Add(new ColoredTextRange(InsertionColors.ForeColor, InsertionColors.BackColor, FindTextIndex, FindText.Length));
						}
						else
						{
							FindTextSegments.Add(new ColoredTextRange(InsertionColors.ForeColor, InsertionColors.BackColor, StartIndex, (FindTextIndex + FindText.Length) - StartIndex));
						}
					}

					FindTextIndex = LineText.IndexOf(FindText, FindTextIndex + FindText.Length, ComparisonType);
				}

				bHasFindText = FindTextSegments.Count > 0;

				int CurrentFindTextSegmentIndex = 0;
				bool bHasOverlap = false;

				foreach(ColoredTextRange TextRange in this.mColorRanges)
				{
					if(TextRange.StartIndex < StartIndex && TextRange.StartIndex + TextRange.Length <= StartIndex)
					{
						continue;
					}
					else if(TextRange.StartIndex >= EndLineIndex)
					{
						break;
					}

					int CurTextRangeStart = Math.Max(StartIndex, TextRange.StartIndex);

					if(CurrentFindTextSegmentIndex < FindTextSegments.Count)
					{
						int CurTextRangeEnd = Math.Min(TextRange.StartIndex + TextRange.Length, EndLineIndex);

						// check to see if it's an overlapping find segment
						if(bHasOverlap)
						{
							// if it is then it has already been added so reassign the start point
							CurTextRangeStart = FindTextSegments[CurrentFindTextSegmentIndex].StartIndex + FindTextSegments[CurrentFindTextSegmentIndex].Length;
							++CurrentFindTextSegmentIndex;
							bHasOverlap = false;
						}

						while(CurrentFindTextSegmentIndex < FindTextSegments.Count && CurTextRangeStart < EndLineIndex && FindTextSegments[CurrentFindTextSegmentIndex].StartIndex >= CurTextRangeStart
							&& FindTextSegments[CurrentFindTextSegmentIndex].StartIndex < CurTextRangeEnd)
						{
							if(FindTextSegments[CurrentFindTextSegmentIndex].StartIndex > CurTextRangeStart)
							{
								Segments.Add(new ColoredDocumentLineSegment(new ColorPair(TextRange.BackColor, TextRange.ForeColor), this.ToString(CurTextRangeStart, Math.Min(FindTextSegments[CurrentFindTextSegmentIndex].StartIndex - CurTextRangeStart, EndLineIndex - CurTextRangeStart))));
							}

							if(FindTextSegments[CurrentFindTextSegmentIndex].StartIndex < EndLineIndex)
							{
								Segments.Add(new ColoredDocumentLineSegment(InsertionColors, this.ToString(FindTextSegments[CurrentFindTextSegmentIndex].StartIndex, Math.Min(FindTextSegments[CurrentFindTextSegmentIndex].Length, EndLineIndex - FindTextSegments[CurrentFindTextSegmentIndex].StartIndex))));
							}

							CurTextRangeStart = FindTextSegments[CurrentFindTextSegmentIndex].StartIndex + FindTextSegments[CurrentFindTextSegmentIndex].Length;

							// if the beginning of the next segment is within the current text range move onto the next "find" segment
							// if not that means the current "find" text segment overlaps into the next regular text segment
							if(CurTextRangeStart < CurTextRangeEnd)
							{
								++CurrentFindTextSegmentIndex;
							}
							else
							{
								bHasOverlap = true;
							}
						}

						// if there was text remaining in the current text range after the last segment then add it now
						if(CurTextRangeStart < CurTextRangeEnd && CurTextRangeStart < EndLineIndex)
						{
							Segments.Add(new ColoredDocumentLineSegment(new ColorPair(TextRange.BackColor, TextRange.ForeColor), this.ToString(CurTextRangeStart, Math.Min(CurTextRangeEnd - CurTextRangeStart, EndLineIndex - CurTextRangeStart))));
						}
					}
					else
					{
						Segments.Add(new ColoredDocumentLineSegment(new ColorPair(TextRange.BackColor, TextRange.ForeColor), this.ToString(CurTextRangeStart, Math.Min(TextRange.Length, EndLineIndex - CurTextRangeStart))));
					}
				}

				return Segments.ToArray();
			}

			/// <summary>
			/// Finishes construction of the line so that reads and searches may be optimized.
			/// </summary>
			public void FinishBuilding()
			{
				if(mFinalText == null)
				{
					mFinalText = mBldrText.ToString();
					mBldrText = null;
				}
			}

			/// <summary>
			/// Returns the text associated with the line.
			/// </summary>
			/// <returns>The text associated with the line.</returns>
			public override string ToString()
			{
				return mFinalText == null ? mBldrText.ToString() : mFinalText;
			}

			/// <summary>
			/// Returns the text associated with the line.
			/// </summary>
			/// <param name="StartIndex">The character index that the returned text will start at.</param>
			/// <param name="Length">The number of characters to return.</param>
			/// <returns>A string containing the text within the specified range of the line.</returns>
			public string ToString(int StartIndex, int Length)
			{
				if(mFinalText == null)
				{
					return mBldrText.ToString(StartIndex, Length);
				}

				return mFinalText.Substring(StartIndex, Length);
			}

			#region ICloneable Members

			public object Clone()
			{
				DocumentLine NewLine = new DocumentLine();

				if(mBldrText != null)
				{
					NewLine.mBldrText.Append(mBldrText.ToString());
				}
				else
				{
					NewLine.mBldrText = null;
				}

				NewLine.mFinalText = mFinalText;
				NewLine.mColorRanges = new List<ColoredTextRange>(mColorRanges);

				if(mUserData != null)
				{
					NewLine.mUserData = mUserData.Clone() as ICloneable;
				}

				return NewLine;
			}

			#endregion
		}

		/// <summary>
		/// This structure represents a read only document line.
		/// </summary>
		/// <remarks>
		/// This is a structure for performance reasons. This will be allocated for every line that is completed for the OnLineAdded event and if it was a class it would increase GC pressure.
		/// </remarks>
		public struct ReadOnlyDocumentLine
		{
			DocumentLine mDocLine;

			internal ReadOnlyDocumentLine(DocumentLine DocLine)
			{
				mDocLine = DocLine;
			}

			/// <summary>
			/// Gets the length of the line in characters.
			/// </summary>
			public int Length
			{
				get { return mDocLine.Length; }
			}

			/// <summary>
			/// Gets/Sets the user data associated with the line.
			/// </summary>
			public ICloneable UserData
			{
				get { return mDocLine.UserData; }
				set { mDocLine.UserData = value; }
			}

			/// <summary>
			/// Gets the character at the specified index.
			/// </summary>
			/// <param name="Index">The index of the character to get.</param>
			/// <returns>The character at the specified index.</returns>
			public char this[int Index]
			{
				get { return mDocLine[Index]; }
			}

			/// <summary>
			/// Gets an array of colored line segments that can be used for drawing.
			/// </summary>
			/// <returns>An array of colored line segments.</returns>
			public ColoredDocumentLineSegment[] GetLineSegments()
			{
				return mDocLine.GetLineSegments();
			}

			/// <summary>
			/// Gets an array of colored line segments that can be used for drawing.
			/// </summary>
			/// <param name="StartIndex">The starting character at which to begin generating segments.</param>
			/// <param name="Length">The number of characters to include in segment generation.</param>
			/// <returns>An array of colored line segments.</returns>
			public ColoredDocumentLineSegment[] GetLineSegments(int StartIndex, int Length)
			{
				return mDocLine.GetLineSegments(StartIndex, Length);
			}

			/// <summary>
			/// Returns the text associated with the line.
			/// </summary>
			/// <returns>The text associated with the line.</returns>
			public override string ToString()
			{
				return mDocLine.ToString();
			}

			/// <summary>
			/// Returns the text associated with the line.
			/// </summary>
			/// <param name="StartIndex">The character index that the returned text will start at.</param>
			/// <param name="Length">The number of characters to return.</param>
			/// <returns>A string containing the text within the specified range of the line.</returns>
			public string ToString(int StartIndex, int Length)
			{
				return mDocLine.ToString(StartIndex, Length);
			}
		}

		List<DocumentLine> mLines = new List<DocumentLine>();
		int mLongestLineLength;

		EventHandler<OutputWindowDocumentLineAddedEventArgs> mOnLineAdded;
		EventHandler<EventArgs> mOnModified;

		/// <summary>
		/// Triggered when a full line of text has been added to the document.
		/// </summary>
		public event EventHandler<OutputWindowDocumentLineAddedEventArgs> LineAdded
		{
			add { mOnLineAdded += value; }
			remove { mOnLineAdded -= value; }
		}

		/// <summary>
		/// Triggered when the document has been modified.
		/// </summary>
		public event EventHandler<EventArgs> Modified
		{
			add { mOnModified += value; }
			remove { mOnModified -= value; }
		}

		/// <summary>
		/// Gets/Sets the text in the document.
		/// </summary>
		public string Text
		{
			set
			{
				if(value == null)
				{
					throw new ArgumentNullException("value");
				}

				Clear();

				AppendText(null, value);
			}
			get
			{
				StringBuilder Bldr = new StringBuilder();

				foreach(DocumentLine CurLine in mLines)
				{
					Bldr.Append(CurLine.ToString());
				}

				return Bldr.ToString();
			}
		}

		/// <summary>
		/// Gets the lines associated with the document.
		/// </summary>
		public string[] Lines
		{
			get
			{
				string[] Ret = new string[mLines.Count];

				for(int i = 0; i < mLines.Count; ++i)
				{
					Ret[i] = mLines[i].ToString();
				}

				return Ret;
			}
		}

		/// <summary>
		/// Gets the number of lines in the document.
		/// </summary>
		public int LineCount
		{
			get { return mLines.Count; }
		}

		/// <summary>
		/// Gets the length in characters of the longest line in the document.
		/// </summary>
		public int LongestLineLength
		{
			get { return mLongestLineLength; }
		}

		/// <summary>
		/// Gets the <see cref="TextLocation"/> of the beginning of the document.
		/// </summary>
		public TextLocation BeginningOfDocument
		{
			get { return new TextLocation(0, 0); }
		}

		/// <summary>
		/// Gets the <see cref="TextLocation"/> of the end of the document.
		/// </summary>
		public TextLocation EndOfDocument
		{
			get
			{
				int LineIndex = Math.Max(0, mLines.Count - 1);
				return new TextLocation(LineIndex, GetLineLength(LineIndex));
			}
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		public OutputWindowDocument()
		{
			// The document always has at least 1 line.
			mLines.Add(new DocumentLine());
		}

		/// <summary>
		/// Appends text of the specified color to the document.
		/// </summary>
		/// <param name="TxtColor">The color of the text to be appended</param>
		/// <param name="Txt">The text to be appended</param>
		public void AppendText(Color? TxtColor, string Txt)
		{
			if(Txt.Length == 0)
			{
				return;
			}

			DocumentLine CurLine;
			int CurLineLength = 0;

			if(mLines.Count == 0)
			{
				CurLine = new DocumentLine();
				mLines.Add(CurLine);
			}
			else
			{
				CurLine = mLines[mLines.Count - 1];
			}

			char LastChar = (char)0;
            // Break the Text in to multiple lines
            String[] Lines = System.Text.RegularExpressions.Regex.Split(Txt, "\r\n");
            foreach (String Line in Lines)
            {
                // Skip empty lines
                if (Line.Length == 0)
                {
                    continue;
                }
                // Put back the \r\n
                String ThisLine = Line + "\r\n";
                Color? CurrColor = TxtColor;
                // Check for keywords
                if (ThisLine.Contains("Error"))
                {
                    CurrColor = Color.Red;
                }
                else if (ThisLine.Contains("Warning"))
                {
                    CurrColor = Color.Orange;
                }
                // Append the line
                foreach (char CurChar in ThisLine)
                {
                    if (CurChar == '\n' && LastChar != '\r')
                    {
                        CurLine.Append(CurrColor, '\r');
                    }

                    // replace tabs with 4 spaces
                    if (CurChar == '\t')
                    {
                        CurLine.Append(CurrColor, "    ");
                    }
                    else
                    {
                        CurLine.Append(CurrColor, CurChar);
                    }

                    if (CurChar == '\n')
                    {
                        // only count displayable characters (cut \r\n)
                        CurLineLength = CurLine.Length - 2;

                        if (CurLineLength > mLongestLineLength)
                        {
                            mLongestLineLength = CurLineLength;
                        }

                        // This isn't required but it enables readonly optimizations
                        CurLine.FinishBuilding();

                        // A full line has been created so trigger the line added event
                        OnLineAdded(new OutputWindowDocumentLineAddedEventArgs(mLines.Count - 1, new ReadOnlyDocumentLine(CurLine)));

                        // Then begin our new line
                        CurLine = new DocumentLine();
                        mLines.Add(CurLine);
                    }

                    LastChar = CurChar;
                }
            }

			// NOTE: This line hasn't been terminated yet so we don't have to account for \r\n at the end
			CurLineLength = CurLine.Length;

			if(CurLineLength > mLongestLineLength)
			{
				mLongestLineLength = CurLineLength;
			}

			OnModified(new EventArgs());
		}

		/// <summary>
		/// Appends a line of colored text to the document.
		/// </summary>
		/// <param name="TxtColor">The color of the text to be appended.</param>
		/// <param name="Txt">The text to append.</param>
		public void AppendLine(Color? TxtColor, string Txt)
		{
			AppendText(TxtColor, Txt + Environment.NewLine);
		}

		/// <summary>
		/// Retrieves a line of text from the document.
		/// </summary>
		/// <param name="Index">The index of the line to retrieve.</param>
		/// <param name="bIncludeEOL">True if the line includes the EOL characters.</param>
		/// <returns>A string containing the text of the specified line.</returns>
		public string GetLine(int Index, bool bIncludeEOL)
		{
			DocumentLine Line = mLines[Index];

			if(!bIncludeEOL && Line.Length >= 2 && Line[Line.Length - 1] == '\n' && Line[Line.Length - 2] == '\r')
			{
				return Line.ToString(0, Line.Length - 2);
			}

			return Line.ToString();
		}

		/// <summary>
		/// Retrieves a line of text from the document.
		/// </summary>
		/// <param name="LineIndex">The index of the line to retrieve.</param>
		/// <param name="CharOffset">The character to start copying.</param>
		/// <param name="Length">The number of characters to copy.</param>
		/// <returns>A string containing the specified range of text text in the specified line.</returns>
		public string GetLine(int LineIndex, int CharOffset, int Length)
		{
			return mLines[LineIndex].ToString(CharOffset, Length);
		}

		/// <summary>
		/// Gets the length of the line in characters at the specified index.
		/// </summary>
		/// <param name="Index">The index of the line to retrieve length information for.</param>
		/// <returns>The length of the line in characters at the specified index.</returns>
		public int GetLineLength(int Index)
		{
			if(Index >= 0 && Index < mLines.Count)
			{
				return GetLineLength(mLines[Index]);
			}

			return 0;
		}

		/// <summary>
		/// Gets the length of the line in characters.
		/// </summary>
		/// <param name="Line">The line whose length is to be retrieved.</param>
		/// <returns>The length of the line with the EOL characters removed (if they exist).</returns>
		private static int GetLineLength(DocumentLine Line)
		{
			if(Line.Length >= 2 && Line[Line.Length - 1] == '\n' && Line[Line.Length - 2] == '\r')
			{
				return Line.Length - 2;
			}
			else
			{
				return Line.Length;
			}
		}

		/// <summary>
		/// Clears the document.
		/// </summary>
		public void Clear()
		{
			mLines.Clear();
			mLongestLineLength = 0;

			OnModified(new EventArgs());
		}

		/// <summary>
		/// Finds the first occurence of the specified string within the document if it exists.
		/// </summary>
		/// <param name="Txt">The string to find.</param>
		/// <param name="StartLoc">The position within the document to begin searching.</param>
		/// <param name="EndLoc">The position within the document to end searching.</param>
		/// <param name="Flags">Flags telling the document how to conduct its search.</param>
		/// <param name="Result">The location within the document of the supplied text.</param>
		/// <returns>True if the text was found.</returns>
		public bool Find(string Txt, TextLocation StartLoc, TextLocation EndLoc, RichTextBoxFinds Flags, out FindResult Result)
		{
			if((Flags & RichTextBoxFinds.Reverse) == RichTextBoxFinds.Reverse)
			{
				return FindReverse(Txt, ref StartLoc, ref EndLoc, Flags, out Result);
			}
			else
			{
				return FindForward(Txt, ref StartLoc, ref EndLoc, Flags, out Result);
			}
		}

		/// <summary>
		/// Looks forward in the document from the specified location for the supplied text.
		/// </summary>
		/// <param name="Txt">The text to find.</param>
		/// <param name="StartLoc">The location to start searching from.</param>
		/// <param name="EndLoc">The location to stop searching at.</param>
		/// <param name="Flags">Flags that control how the search is performed.</param>
		/// <param name="Result">Receives the result.</param>
		/// <returns>True if a match was found.</returns>
		private bool FindForward(string Txt, ref TextLocation StartLoc, ref TextLocation EndLoc, RichTextBoxFinds Flags, out FindResult Result)
		{
			bool bMatchWord;
			bool bFound = false;
			bool bIsWord;
			StringComparison ComparisonFlags;

			SetupFindState(Txt, ref StartLoc, ref EndLoc, Flags, out Result, out bIsWord, out ComparisonFlags, out bMatchWord);

			for(int CurLineIndex = StartLoc.Line; CurLineIndex <= EndLoc.Line && !bFound; ++CurLineIndex)
			{
				if(GetLineLength(CurLineIndex) == 0)
				{
					continue;
				}

				DocumentLine CurLineBldr = mLines[CurLineIndex];
				string LineTxt;
				int ColumnIndex = 0;

				if(CurLineIndex == StartLoc.Line && StartLoc.Column > 0)
				{
					LineTxt = CurLineBldr.ToString(StartLoc.Column, CurLineBldr.Length - StartLoc.Column);
					ColumnIndex = StartLoc.Column;
				}
				else if(CurLineIndex == EndLoc.Line && EndLoc.Column < GetLineLength(CurLineIndex))
				{
					LineTxt = CurLineBldr.ToString(0, EndLoc.Column + 1);
				}
				else
				{
					LineTxt = CurLineBldr.ToString();
				}

				int Index = LineTxt.IndexOf(Txt, ComparisonFlags);

				if(Index != -1)
				{
					ColumnIndex += Index;
					CheckForWholeWord(Txt, ref Result, bMatchWord, ref bFound, bIsWord, CurLineIndex, CurLineBldr, ColumnIndex, Index);
				}
			}

			return bFound;
		}

		/// <summary>
		/// Checks to see if a text location is a matching word.
		/// </summary>
		/// <param name="Txt">The text being searched for.</param>
		/// <param name="Result">Receives the result if the text location is a matching word.</param>
		/// <param name="bMatchWord">True if an entire word is to be matched.</param>
		/// <param name="bFound">Set to true if a matching word is found.</param>
		/// <param name="bIsWord">True if <paramref name="Txt"/> is a valid word.</param>
		/// <param name="CurLineIndex">The index of the current line.</param>
		/// <param name="CurLineBldr">The text of the current line.</param>
		/// <param name="ColumnIndex">The character index within the line of text where the matching will begin.</param>
		/// <param name="Index">The index of a match within the range of searchable characters for the current line. The true line index is <paramref name="ColumnIndex"/> + <paramref name="Index"/>.</param>
		private static void CheckForWholeWord(string Txt, ref FindResult Result, bool bMatchWord, ref bool bFound, bool bIsWord, int CurLineIndex, DocumentLine CurLine, int ColumnIndex, int Index)
		{
			int FinalCharIndex = ColumnIndex + Txt.Length;

			if(bMatchWord && bIsWord)
			{
				if((FinalCharIndex >= CurLine.Length || !IsWordCharacter(CurLine[FinalCharIndex]))
					&& (ColumnIndex == 0 || !IsWordCharacter(CurLine[ColumnIndex - 1])))
				{
					bFound = true;
				}
			}
			else
			{
				bFound = true;
			}

			if(bFound)
			{
				Result.Line = CurLineIndex;
				Result.Column = ColumnIndex;
				Result.Length = Txt.Length;
			}
		}

		/// <summary>
		/// Checks to see if a character is valid within a word.
		/// </summary>
		/// <param name="CharToCheck">The character to check.</param>
		/// <returns>True if the character is valid within a word.</returns>
		private static bool IsWordCharacter(char CharToCheck)
		{
			return char.IsLetterOrDigit(CharToCheck) || CharToCheck == '_';
		}

		/// <summary>
		/// Performs general housekeeping for setting up a search.
		/// </summary>
		/// <param name="Txt">The text to search for.</param>
		/// <param name="StartLoc">The location to begin searching from.</param>
		/// <param name="EndLoc">The location to stop searching at.</param>
		/// <param name="Flags">Flags controlling how the search is performed.</param>
		/// <param name="Result">Receives the resulting location if a match is found.</param>
		/// <param name="bIsWord">Set to true if <paramref name="Txt"/> is a valid word.</param>
		/// <param name="ComparisonFlags">Receives flags controlling how strings are compared.</param>
		/// <param name="bMatchWord">Is set to true if only full words are to be matched.</param>
		private void SetupFindState(string Txt, ref TextLocation StartLoc, ref TextLocation EndLoc, RichTextBoxFinds Flags, out FindResult Result, out bool bIsWord, out StringComparison ComparisonFlags, out bool bMatchWord)
		{
			Result = FindResult.Empty;

			if(!IsValidTextLocation(StartLoc))
			{
				throw new ArgumentException("StartLoc is an invalid text location!");
			}

			if(!IsValidTextLocation(EndLoc))
			{
				throw new ArgumentException("EndLoc is an invalid text location!");
			}

			if((Flags & RichTextBoxFinds.Reverse) == RichTextBoxFinds.Reverse)
			{
				if(StartLoc < EndLoc)
				{
					throw new ArgumentException("StartLoc must be greater than EndLoc when doing a reverse search!");
				}
			}
			else
			{
				if(StartLoc > EndLoc)
				{
					throw new ArgumentException("StartLoc must be less than EndLoc when doing a forward search!");
				}
			}

			bMatchWord = (Flags & RichTextBoxFinds.WholeWord) == RichTextBoxFinds.WholeWord;
			bIsWord = IsWord(0, Txt);
			ComparisonFlags = StringComparison.OrdinalIgnoreCase;

			if((Flags & RichTextBoxFinds.MatchCase) == RichTextBoxFinds.MatchCase)
			{
				ComparisonFlags = StringComparison.Ordinal;
			}
		}

		/// <summary>
		/// Checks a string to see if it contains a valid word.
		/// </summary>
		/// <param name="Index">The index to start validating at.</param>
		/// <param name="Txt">The text to be validated.</param>
		/// <returns>True if all text including and after <paramref name="Index"/> are part of a valid word.</returns>
		private static bool IsWord(int Index, string Txt)
		{
			for(; Index < Txt.Length; ++Index)
			{
				char CurChar = Txt[Index];

				if(!char.IsLetterOrDigit(CurChar) && CurChar != '_')
				{
					return false;
				}
			}

			return true;
		}

		/// <summary>
		/// Searches for a string in the reverse direction of <see cref="FindForward"/>.
		/// </summary>
		/// <param name="Txt">The text to search for.</param>
		/// <param name="StartLoc">The starting location of the search.</param>
		/// <param name="EndLoc">The ending location of the search.</param>
		/// <param name="Flags">Flags controlling how the searching is conducted.</param>
		/// <param name="Result">Receives the results of the search.</param>
		/// <returns>True if a match was found.</returns>
		private bool FindReverse(string Txt, ref TextLocation StartLoc, ref TextLocation EndLoc, RichTextBoxFinds Flags, out FindResult Result)
		{
			bool bFound = false;
			bool bMatchWord;
			bool bIsWord;
			StringComparison ComparisonFlags;

			SetupFindState(Txt, ref StartLoc, ref EndLoc, Flags, out Result, out bIsWord, out ComparisonFlags, out bMatchWord);

			for(int CurLineIndex = StartLoc.Line; CurLineIndex >= EndLoc.Line && !bFound; --CurLineIndex)
			{
				if(GetLineLength(CurLineIndex) == 0)
				{
					continue;
				}

				DocumentLine CurLineBldr = mLines[CurLineIndex];
				string LineTxt;
				int ColumnIndex = 0;

				if(CurLineIndex == StartLoc.Line && StartLoc.Column < GetLineLength(CurLineIndex))
				{
					LineTxt = CurLineBldr.ToString(0, StartLoc.Column);
				}
				else if(CurLineIndex == EndLoc.Line && EndLoc.Column > 0)
				{
					LineTxt = CurLineBldr.ToString(EndLoc.Column, CurLineBldr.Length - EndLoc.Column);
					ColumnIndex = EndLoc.Column;
				}
				else
				{
					LineTxt = CurLineBldr.ToString();
				}

				int Index = LineTxt.LastIndexOf(Txt, ComparisonFlags);

				if(Index != -1)
				{
					ColumnIndex += Index;
					CheckForWholeWord(Txt, ref Result, bMatchWord, ref bFound, bIsWord, CurLineIndex, CurLineBldr, ColumnIndex, Index);
				}
			}

			return bFound;
		}

		/// <summary>
		/// Saves the document to disk.
		/// </summary>
		/// <param name="Name">The path to a file that the document will be written to.</param>
		public void SaveToFile(string Name)
		{
			using(StreamWriter Writer = new StreamWriter(File.Open(Name, FileMode.Create, FileAccess.Write, FileShare.Read)))
			{
				foreach(DocumentLine CurLine in mLines)
				{
					Writer.Write(CurLine.ToString());
				}
			}
		}

		/// <summary>
		/// Determines whether a location within the document is valid.
		/// </summary>
		/// <param name="Loc">The location to validate.</param>
		/// <returns>True if the supplied location is within the bounds of the document.</returns>
		public bool IsValidTextLocation(TextLocation Loc)
		{
			bool bResult = false;

			if(Loc.Line >= 0 && Loc.Line < mLines.Count && Loc.Column >= 0 && Loc.Column <= GetLineLength(Loc.Line))
			{
				bResult = true;
			}

			return bResult;
		}

		/// <summary>
		/// Gets an array of line segments for the line at the specified index.
		/// </summary>
		/// <param name="LineIndex">The index of the line to retrieve the segments for.</param>
		/// <returns>An array of line segments.</returns>
		public ColoredDocumentLineSegment[] GetLineSegments(int LineIndex)
		{
			return mLines[LineIndex].GetLineSegments();
		}

		/// <summary>
		/// Gets an array of line segments for the line at the specified index.
		/// </summary>
		/// <param name="LineIndex">The index of the line to retrieve the segments for.</param>
		/// <param name="StartIndex">The starting character at which to begin generating segments.</param>
		/// <param name="Length">The number of characters to include in segment generation.</param>
		/// <returns>An array of line segments.</returns>
		public ColoredDocumentLineSegment[] GetLineSegments(int LineIndex, int StartIndex, int Length)
		{
			return mLines[LineIndex].GetLineSegments(StartIndex, Length);
		}

		/// <summary>
		/// Gets an array of colored line segments that can be used for drawing.
		/// </summary>
		/// <remarks>
		/// If the line of text contains any instances of <paramref name="FindText"/> they will be replaced with new line segments that contain <paramref name="InsertionColors"/> for drawing.
		/// </remarks>
		/// <param name="LineIndex">The index of the line to retrieve the segments for.</param>
		/// <param name="StartIndex">The starting character at which to begin generating segments.</param>
		/// <param name="Length">The number of characters to include in segment generation.</param>
		/// <param name="FindText">Text to search for that requires special coloring.</param>
		/// <param name="InsertionColors">The colors to use when drawing <paramref name="FindText"/>.</param>
		/// <param name="ComparisonType">The type of string comparison that will be conducted when searching for <paramref name="FindText"/>.</param>
		/// <param name="bHasFindText">Set to True if <paramref name="FindText"/> exists within the line segments.</param>
		/// <returns>An array of colored line segments.</returns>
		public ColoredDocumentLineSegment[] GetLineSegmentsWithFindString(int LineIndex, int StartIndex, int Length, string FindText, ColorPair InsertionColors, StringComparison ComparisonType, out bool bHasFindText)
		{
			return mLines[LineIndex].GetLineSegmentsWithFindString(StartIndex, Length, FindText, InsertionColors, ComparisonType, out bHasFindText);
		}

		/// <summary>
		/// Event handler for when a full line of text has been added to the document.
		/// </summary>
		/// <param name="e">Information about the event.</param>
		protected virtual void OnLineAdded(OutputWindowDocumentLineAddedEventArgs e)
		{
			if(mOnLineAdded != null)
			{
				mOnLineAdded(this, e);
			}
		}

		/// <summary>
		/// Event handler for when the document has been modified.
		/// </summary>
		/// <param name="e">Information about the event.</param>
		protected virtual void OnModified(EventArgs e)
		{
			if(mOnModified != null)
			{
				mOnModified(this, e);
			}
		}

		/// <summary>
		/// Creates a new document by filtering out the lines of the current document using the supplied filter function.
		/// </summary>
		/// <param name="Filter">The function that will filter the lines out of the current document.</param>
		/// <param name="Data">User supplied data that may control filtering.</param>
		/// <returns>A new document containing filtered lines from the current document.</returns>
		public OutputWindowDocument CreateFilteredDocument(OutputWindowDocumentFilterDelegate Filter, object Data)
		{
			if(Filter == null)
			{
				throw new ArgumentNullException("Filter");
			}

			OutputWindowDocument NewDoc = new OutputWindowDocument();

			NewDoc.mLines.Capacity = mLines.Capacity;
			DocumentLine LastLine = null;

			foreach(DocumentLine CurLine in mLines)
			{
				if(!Filter(new ReadOnlyDocumentLine(CurLine), Data))
				{
					DocumentLine NewLine = (DocumentLine)CurLine.Clone();
					NewDoc.mLines.Add(NewLine);

					int LineLength = GetLineLength(NewLine);

					if(LineLength > NewDoc.mLongestLineLength)
					{
						NewDoc.mLongestLineLength = LineLength;
					}

					LastLine = NewLine;
				}
			}

			// If the last line is a full line we need to append an empty line because the empty line has been filtered out
			if(LastLine != null && LastLine.ToString().EndsWith(Environment.NewLine))
			{
				NewDoc.mLines.Add(new DocumentLine());
			}

			return NewDoc;
		}
	}
}