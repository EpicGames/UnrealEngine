// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;

namespace UnrealControls
{
	/// <summary>
	/// This class represents event arguments for when a full line of text has been added to a document.
	/// </summary>
	public class OutputWindowDocumentLineAddedEventArgs : EventArgs
	{
		OutputWindowDocument.ReadOnlyDocumentLine mLine;
		int mLineIndex;

		/// <summary>
		/// Gets the text for the line.
		/// </summary>
		public OutputWindowDocument.ReadOnlyDocumentLine Line
		{
			get { return mLine; }
		}

		/// <summary>
		/// Gets the line index within the document.
		/// </summary>
		public int Index
		{
			get { return mLineIndex; }
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="LineIndex">The index of the line within its document.</param>
		/// <param name="Txt">The line of text.</param>
		public OutputWindowDocumentLineAddedEventArgs(int LineIndex, OutputWindowDocument.ReadOnlyDocumentLine Txt)
		{
			mLine = Txt;
			mLineIndex = LineIndex;
		}
	}
}
