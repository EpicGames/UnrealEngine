// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Drawing;
using System.Data;
using System.Text;
using System.Windows.Forms;
using System.Windows.Forms.VisualStyles;
using System.Runtime.InteropServices;
using System.IO;

namespace UnrealControls
{
	/// <summary>
	/// This class represents a view into an output document.
	/// </summary>
	public partial class OutputWindowView : UserControl
	{
		const TextFormatFlags TXT_FLAGS = TextFormatFlags.NoPadding | TextFormatFlags.NoPrefix | TextFormatFlags.PreserveGraphicsClipping;
		static readonly Color INVALID_SEARCH_BACKCOLOR = Color.FromArgb(255, 102, 102);
		static readonly Color INVALID_SEARCH_FORECOLOR = Color.White;
		static readonly Color HAS_FINDTEXT_BACKCOLOR = Color.FromArgb(239, 248, 255);

		//event delegates
		EventHandler<EventArgs> mOnDocChanged;
		EventHandler<EventArgs> mOnSelectionChanged;

		// controls
		XButton mXButtonFind;
		CheckBox mCheckBox_MatchCase;

		//regular local variables
		OutputWindowDocument mDocument;
		Size mFontDimensions;
		TextLocation mCaretPosition;
		TextLocation mSelectionStart;
		TextLocation mCurrentDocumentSize;
		Point mLastMousePos;
		Color mFindTextLineHighlight = HAS_FINDTEXT_BACKCOLOR;
		Color mFindTextBackColor = Color.Yellow;
		Color mFindTextForeColor = Color.Black;
		Size mLastSize;
		bool mIsSelecting;
		bool mAutoScroll;

		const int VK_SHIFT = 0x10;

#if !__MonoCS__
		[DllImport("user32.dll")]
		extern static ushort GetAsyncKeyState(int vKey);
#endif

		/// <summary>
		/// Gets the maximum number of lines that can be drawn in the control.
		/// </summary>
		private int DisplayableVerticalLines
		{
			get
			{
				int ClientHeight = (this.Size.IsEmpty ? mLastSize.Height : this.Height) - mHScrollBar.Height;

				if(mToolStrip_Find.Visible)
				{
					ClientHeight -= mToolStrip_Find.Height;
				}

				return ClientHeight / this.Font.Height;
			}
		}

		/// <summary>
		/// Gets the width of the actual client area in pixels.
		/// </summary>
		private int WidthInDisplayablePixels
		{
			get { return (this.Size.IsEmpty ? mLastSize.Width : this.Width) - mVScrollBar.Width; }
		}

		/// <summary>
		/// Gets the maximum number of columns/characters that can be drawn in the control.
		/// </summary>
		private int DisplayableCharacters
		{
			get { return this.WidthInDisplayablePixels / mFontDimensions.Width; }
		}

		/// <summary>
		/// Gets/Sets the document associated with the view.
		/// </summary>
		public OutputWindowDocument Document
		{
			get { return mDocument; }
			set
			{
				if(mDocument != value)
				{
					if(mDocument != null)
					{
						mDocument.Modified -= new EventHandler<EventArgs>(mDocument_Modified);
					}

					mDocument = value;

					if(mDocument != null)
					{
						mDocument.Modified += new EventHandler<EventArgs>(mDocument_Modified);
					}

					// must have this here to explicitly reset scroll bar position
					mHScrollBar.Value = 0;
					mVScrollBar.Value = 0;
					mCaretPosition = mSelectionStart = new TextLocation(0, 0);

					OnDocumentChanged(new EventArgs());
				}
			}
		}

		/// <summary>
		/// Gets/Sets the font used to draw text.
		/// </summary>
		public override Font Font
		{
			get
			{
				return base.Font;
			}
			set
			{
				base.Font = value;

				using(Graphics Gfx = this.CreateGraphics())
				{
					mFontDimensions = TextRenderer.MeasureText(Gfx, " ", this.Font, new Size(short.MaxValue, short.MaxValue), TXT_FLAGS);
					Size Temp = TextRenderer.MeasureText(Gfx, "x", this.Font, new Size(short.MaxValue, short.MaxValue), TXT_FLAGS);

					mFontDimensions.Width = Math.Max(mFontDimensions.Width, Temp.Width);
				}
			}
		}

		/// <summary>
		/// Gets/Sets whether or not the control supports autoscrolling output.
		/// </summary>
		[Description("Controls whether or not the control autoscrolls when text is appended.")]
		public override bool AutoScroll
		{
			get
			{
				return mAutoScroll;
			}
			set
			{
				mAutoScroll = value;
			}
		}

		/// <summary>
		/// Gets/Sets the start location of the currently selected text.
		/// </summary>
		[Browsable(false)]
		public TextLocation SelectionStart
		{
			get { return mSelectionStart; }
			set
			{
				if(IsValidTextLocation(value))
				{
					mSelectionStart = value;
					OnSelectionChanged(new EventArgs());

					Invalidate();
				}
			}
		}

		/// <summary>
		/// Gets/Sets the end location of the currently selected text.
		/// </summary>
		[Browsable(false)]
		public TextLocation SelectionEnd
		{
			get { return mCaretPosition; }
			set
			{
				if(IsValidTextLocation(value))
				{
					mCaretPosition = value;
					OnSelectionChanged(new EventArgs());

					UpdateCaret();
					Invalidate();
				}
			}
		}

		/// <summary>
		/// Gets the currently selected text.
		/// </summary>
		[Browsable(false)]
		public string SelectedText
		{
			get
			{
				if(mDocument == null || mSelectionStart == mCaretPosition)
				{
					return "";
				}

				StringBuilder Bldr = new StringBuilder();
				TextLocation StartLoc;
				TextLocation EndLoc;

				if(mSelectionStart < mCaretPosition)
				{
					StartLoc = mSelectionStart;
					EndLoc = mCaretPosition;
				}
				else
				{
					StartLoc = mCaretPosition;
					EndLoc = mSelectionStart;
				}

				for(int LineIndex = StartLoc.Line; LineIndex <= EndLoc.Line; ++LineIndex)
				{
					string CurLine = mDocument.GetLine(LineIndex, true);
					int EndCol = LineIndex == EndLoc.Line ? EndLoc.Column : CurLine.Length;

					for(int StartCol = LineIndex == StartLoc.Line ? StartLoc.Column : 0; StartCol < EndCol; ++StartCol)
					{
						Bldr.Append(CurLine[StartCol]);
					}
				}

				return Bldr.ToString();
			}
		}

		/// <summary>
		/// Gets whether or not there's currently selected text.
		/// </summary>
		[Browsable(false)]
		public bool HasSelectedText
		{
			get { return mDocument != null && mCaretPosition != mSelectionStart; }
		}

		/// <summary>
		/// Gets/Sets the location of the caret within the source document.
		/// </summary>
		[Browsable(false)]
		public TextLocation CaretLocation
		{
			get { return mCaretPosition; }
			set
			{
				if(IsValidTextLocation(value))
				{
					bool bHadSelection = this.HasSelectedText;

					mSelectionStart = mCaretPosition = value;

					if(bHadSelection)
					{
						OnSelectionChanged(new EventArgs());
					}

					UpdateCaret();
				}
			}
		}

		/// <summary>
		/// Gets/Sets the text for the window.
		/// </summary>
		public override string Text
		{
			get
			{
				if(mDocument != null)
				{
					return mDocument.Text;
				}

				return "";
			}
			set
			{
				if(mDocument != null)
				{
					mDocument.Text = value;
				}
			}
		}

		/// <summary>
		/// Gets whether or not the view is in "find" mode.
		/// </summary>
		public bool IsInFindMode
		{
			get { return mToolStrip_Find.Visible; }
		}

		/// <summary>
		/// Gets/Sets the highlight color used on lines that contain "find" text.
		/// </summary>
		public Color FindTextLineHighlight
		{
			get { return mFindTextLineHighlight; }
			set
			{
				if(mFindTextLineHighlight != value)
				{
					mFindTextLineHighlight = value;
					Invalidate();
				}
			}
		}

		/// <summary>
		/// Gets/Sets the background color used to draw "find" text.
		/// </summary>
		public Color FindTextBackColor
		{
			get { return mFindTextBackColor; }
			set
			{
				if(mFindTextBackColor != value)
				{
					mFindTextBackColor = value;
					Invalidate();
				}
			}
		}

		/// <summary>
		/// Gets/Sets the foreground color used to draw "find" text.
		/// </summary>
		public Color FindTextForeColor
		{
			get { return mFindTextForeColor; }
			set
			{
				if(mFindTextForeColor != value)
				{
					mFindTextForeColor = value;
					Invalidate();
				}
			}
		}

		/// <summary>
		/// Triggered when the document associated with the view has changed.
		/// </summary>
		public event EventHandler<EventArgs> DocumentChanged
		{
			add { mOnDocChanged += value; }
			remove { mOnDocChanged -= value; }
		}

		/// <summary>
		/// Triggered when the selected text has changed.
		/// </summary>
		public event EventHandler<EventArgs> SelectionChanged
		{
			add { mOnSelectionChanged += value; }
			remove { mOnSelectionChanged -= value; }
		}

		#region PInvoke Imports

#if !__MonoCS__
		[DllImport("user32.dll")]
		static extern bool CreateCaret(IntPtr hWnd, IntPtr hBitmap, int nWidth, int nHeight);

		[DllImport("user32.dll")]
		static extern bool DestroyCaret();

		[DllImport("user32.dll")]
		static extern bool ShowCaret(IntPtr hWnd);

		[DllImport("user32.dll")]
		static extern bool SetCaretPos(int X, int Y);

		[DllImport("user32.dll")]
		static extern bool HideCaret(IntPtr hWnd);
#endif

		#endregion

		/// <summary>
		/// Constructor.
		/// </summary>
		public OutputWindowView()
		{
			InitializeComponent();

			mXButtonFind = new XButton();
			mXButtonFind.Size = new Size(16, 15);
			mXButtonFind.Anchor = AnchorStyles.None;
			mXButtonFind.BackColor = SystemColors.MenuBar;
			mXButtonFind.Click += new EventHandler(mXButtonFind_Click);
			mXButtonFind.MouseEnter += new EventHandler(ChildControl_MouseEnter);

			mCheckBox_MatchCase = new CheckBox();
			mCheckBox_MatchCase.Text = "Match Case";
			mCheckBox_MatchCase.BackColor = SystemColors.MenuBar;
			mCheckBox_MatchCase.CheckedChanged += new EventHandler(mToolStripTextBox_Find_TextChanged);
			mCheckBox_MatchCase.MouseEnter += new EventHandler(ChildControl_MouseEnter);
			
			TableLayoutPanel XButtonPanel = new TableLayoutPanel();
			XButtonPanel.BackColor = SystemColors.MenuBar;
			XButtonPanel.ColumnStyles.Add(new ColumnStyle(SizeType.AutoSize));
			XButtonPanel.RowStyles.Add(new RowStyle(SizeType.AutoSize));
			XButtonPanel.Controls.Add(mXButtonFind);
			XButtonPanel.MouseEnter += new EventHandler(ChildControl_MouseEnter);

			ToolStripControlHost XButtonFind_Host = new ToolStripControlHost(XButtonPanel);
			XButtonFind_Host.Padding = new Padding(0, 0, 6, 0);
			
			mToolStrip_Find.Items.Insert(0, XButtonFind_Host);
			mToolStrip_Find.Items.Add(new ToolStripControlHost(mCheckBox_MatchCase));
			// NOTE: We do this here instead of in the designer because we still want it to show up in the designer
			mToolStrip_Find.Visible = false;

			// get the initial font measurement
			this.Font = this.Font;
			this.Document = new OutputWindowDocument();
			mLastSize = this.Size;
		}

		/// <summary>
		/// Event handler for when the X button on the "Find" toolstrip is clicked.
		/// </summary>
		/// <param name="sender">The object that initiated the event.</param>
		/// <param name="e">Additional information about the event.</param>
		void mXButtonFind_Click(object sender, EventArgs e)
		{
			ExitFindMode();
		}

		/// <summary>
		/// Draws the control.
		/// </summary>
		/// <param name="e">Information about the event.</param>
		protected override void OnPaint(PaintEventArgs e)
		{
			base.OnPaint(e);

			try
			{
				if(mDocument != null)
				{
					ColoredDocumentLineSegment[] LineSegmentList = null;
					string FindTxt = mToolStripTextBox_Find.Text.Trim();
					int SelectionStart;
					int SelectionLength;
					int YLineOffset = 0;
					int DisplayableVLines = this.DisplayableVerticalLines;
					System.Drawing.Size MaxLineSize = new System.Drawing.Size(short.MaxValue, short.MaxValue);

					// NOTE: We have to draw background highlights first to prevent drawing over overhanging glyphs from the previous line
					if(this.IsInFindMode && FindTxt.Length > 0)
					{
						using(SolidBrush HighlightBrush = new SolidBrush(mFindTextLineHighlight))
						{
							for(int DocLineIndex = mVScrollBar.Value, VisibleLineIndex = 0; VisibleLineIndex < DisplayableVLines && DocLineIndex < mDocument.LineCount; ++DocLineIndex, ++VisibleLineIndex, YLineOffset += this.Font.Height)
							{
								string FullLineText = mDocument.GetLine(DocLineIndex, false);

								if(FullLineText.IndexOf(FindTxt, mCheckBox_MatchCase.Checked ? StringComparison.Ordinal : StringComparison.OrdinalIgnoreCase) != -1)
								{
									e.Graphics.FillRectangle(HighlightBrush, 0, YLineOffset, this.WidthInDisplayablePixels, this.Font.Height);
								}
							}
						}
					}

					// reset the y offset to begin drawing the lines
					YLineOffset = 0;

					for(int DocLineIndex = mVScrollBar.Value, VisibleLineIndex = 0; VisibleLineIndex < DisplayableVLines && DocLineIndex < mDocument.LineCount; ++DocLineIndex, ++VisibleLineIndex, YLineOffset += this.Font.Height)
					{
						int LineLength = mDocument.GetLineLength(DocLineIndex);
						int XOffset = 0;

						if(mHScrollBar.Value >= LineLength || LineLength == 0)
						{
							continue;
						}
						else if(GetSelectionForLine(DocLineIndex, out SelectionStart, out SelectionLength))
						{
							// first check to see if there's txt to be drawn before the selection
							if(SelectionStart > mHScrollBar.Value)
							{
								LineSegmentList = GetSegmentsForLine(DocLineIndex, mHScrollBar.Value, SelectionStart - mHScrollBar.Value);

								foreach(ColoredDocumentLineSegment CurSegment in LineSegmentList)
								{
									DrawLineSegment(e.Graphics, YLineOffset, ref MaxLineSize, ref XOffset, CurSegment);
								}
							}
							else if(SelectionStart + SelectionLength >= mHScrollBar.Value)
							{
								SelectionLength -= mHScrollBar.Value - SelectionStart;
								SelectionStart = mHScrollBar.Value;
							}
							else
							{
								SelectionStart = mHScrollBar.Value;
								SelectionLength = 0;
							}

							// now draw the selected text
							if(SelectionLength > 0)
							{
								string LineSegment = mDocument.GetLine(DocLineIndex, SelectionStart, SelectionLength);

								Size SelectionRect = TextRenderer.MeasureText(e.Graphics, LineSegment, this.Font, new Size(short.MaxValue, short.MaxValue), TXT_FLAGS);
								e.Graphics.FillRectangle(SystemBrushes.Highlight, XOffset, YLineOffset, SelectionRect.Width, this.Font.Height);

								TextRenderer.DrawText(e.Graphics, LineSegment, this.Font, new Point(XOffset, YLineOffset), SystemColors.HighlightText, TXT_FLAGS);
								XOffset += TextRenderer.MeasureText(e.Graphics, LineSegment, this.Font, MaxLineSize, TXT_FLAGS).Width;
								SelectionStart += SelectionLength;
							}

							// draw the last segment of unselected text if it exists
							if(SelectionStart < LineLength)
							{
								LineSegmentList = GetSegmentsForLine(DocLineIndex, SelectionStart, LineLength - SelectionStart);


								foreach(ColoredDocumentLineSegment CurSegment in LineSegmentList)
								{
									DrawLineSegment(e.Graphics, YLineOffset, ref MaxLineSize, ref XOffset, CurSegment);
								}
							}
						}
						else
						{
							if(mHScrollBar.Value > 0)
							{
								LineSegmentList = GetSegmentsForLine(DocLineIndex, mHScrollBar.Value, LineLength - mHScrollBar.Value);
							}
							else
							{
								LineSegmentList = GetSegmentsForLine(DocLineIndex, 0, LineLength);
							}

							foreach(ColoredDocumentLineSegment CurSegment in LineSegmentList)
							{
								DrawLineSegment(e.Graphics, YLineOffset, ref MaxLineSize, ref XOffset, CurSegment);
							}
						}
					}
				}

				// fill in the space between scroll bars
				e.Graphics.FillRectangle(SystemBrushes.Control, mVScrollBar.Location.X, mHScrollBar.Location.Y, mVScrollBar.Width, mHScrollBar.Height);
			}
			catch(Exception ex)
			{
				System.Diagnostics.Debug.WriteLine(ex.ToString());
			}
		}

		/// <summary>
		/// Gets the segments for a line so that it can be drawn.
		/// </summary>
		/// <param name="LineIndex">The index of the line whose segments are being retrieved.</param>
		/// <param name="ColIndex">The column to start retrieving segments at.</param>
		/// <param name="Length">The number of characters from <paramref name="ColIndex"/> to return segments for.</param>
		/// <returns>An array of segments associated with the line.</returns>
		private ColoredDocumentLineSegment[] GetSegmentsForLine(int LineIndex, int ColIndex, int Length)
		{
			System.Diagnostics.Debug.Assert(mDocument != null);

			bool bHasFindText;
			ColoredDocumentLineSegment[] Segments;

			if(this.IsInFindMode && mToolStripTextBox_Find.Text.Trim().Length > 0)
			{
				Segments = mDocument.GetLineSegmentsWithFindString(LineIndex, ColIndex, Length, mToolStripTextBox_Find.Text, new ColorPair(mFindTextBackColor, mFindTextForeColor), mCheckBox_MatchCase.Checked ? StringComparison.Ordinal : StringComparison.OrdinalIgnoreCase, out bHasFindText);
			}
			else
			{
				Segments = mDocument.GetLineSegments(LineIndex, ColIndex, Length);
			}

			return Segments;
		}

		/// <summary>
		/// Draws the text for a line segment.
		/// </summary>
		/// <param name="Gfx">The <see cref="Graphics"/> object to use for drawing.</param>
		/// <param name="YLineOffset">The Y location to begin drawing.</param>
		/// <param name="MaxLineSize">The maximum allowed line size.</param>
		/// <param name="XOffset">The X location to begin drawing.</param>
		/// <param name="CurSegment">The segment to be drawn.</param>
		private void DrawLineSegment(Graphics Gfx, int YLineOffset, ref System.Drawing.Size MaxLineSize, ref int XOffset, ColoredDocumentLineSegment CurSegment)
		{
			ColorPair Colors = CurSegment.Color;
			int SegmentWidth = TextRenderer.MeasureText(Gfx, CurSegment.Text, this.Font, MaxLineSize, TXT_FLAGS).Width;

			if(Colors.BackColor.HasValue)
			{
				TextRenderer.DrawText(Gfx, CurSegment.Text, this.Font, new Point(XOffset, YLineOffset), CurSegment.Color.ForeColor.HasValue ? CurSegment.Color.ForeColor.Value : this.ForeColor, CurSegment.Color.BackColor.Value, TXT_FLAGS);
			}
			else
			{
				TextRenderer.DrawText(Gfx, CurSegment.Text, this.Font, new Point(XOffset, YLineOffset), CurSegment.Color.ForeColor.HasValue ? CurSegment.Color.ForeColor.Value : this.ForeColor, TXT_FLAGS);
			}

			XOffset += SegmentWidth;
		}

		/// <summary>
		/// Event handler for when the document associated with the view has been changed.
		/// </summary>
		/// <param name="e">Information about the event.</param>
		virtual protected void OnDocumentChanged(EventArgs e)
		{
			if(mDocument != null)
			{
				mCurrentDocumentSize.Line = mDocument.LineCount;
				mCurrentDocumentSize.Column = mDocument.GetLineLength(mCurrentDocumentSize.Line - 1);
			}
			else
			{
				mCurrentDocumentSize.Column = 0;
				mCurrentDocumentSize.Line = 0;
			}

			UpdateCaret();
			UpdateScrollbars();
			Invalidate();

			if(mOnDocChanged != null)
			{
				mOnDocChanged(this, e);
			}
		}

		/// <summary>
		/// Event handler for when the selected text has changed.
		/// </summary>
		/// <param name="e">Information about the event.</param>
		virtual protected void OnSelectionChanged(EventArgs e)
		{
			if(mOnSelectionChanged != null)
			{
				mOnSelectionChanged(this, e);
			}
		}

		/// <summary>
		/// Updates the vertical and horizontal scrollbar values.
		/// </summary>
		private void UpdateScrollbars()
		{
			if(mDocument == null)
			{
				mVScrollBar.Value = 0;
				mVScrollBar.Maximum = 0;
				mVScrollBar.Enabled = false;

				mHScrollBar.Value = 0;
				mHScrollBar.Maximum = 0;
				mHScrollBar.Enabled = false;
			}
			else
			{
				// update horizontal scrollbar
				int HorizontalDisplayableCount = this.DisplayableCharacters;
				mHScrollBar.Maximum = Math.Max(0, mDocument.LongestLineLength - 1);
				mHScrollBar.LargeChange = Math.Max(0, HorizontalDisplayableCount);

				int MaxPos = mHScrollBar.Maximum - HorizontalDisplayableCount;

				if(MaxPos < 0)
				{
					MaxPos = 0;
				}

				if(mHScrollBar.Value > MaxPos)
				{
					mHScrollBar.Value = MaxPos;
				}

				mHScrollBar.Enabled = HorizontalDisplayableCount < mDocument.LongestLineLength;

				// update vertical scrollbar
				int VerticalDisplayableCount = this.DisplayableVerticalLines;

				// must subtract 1 or else it displays an extra line
				mVScrollBar.Maximum = Math.Max(0, mDocument.LineCount - 1);
				mVScrollBar.LargeChange = Math.Max(0, VerticalDisplayableCount);

				MaxPos = GetMaximumVScrollValue();

				if(MaxPos < 0)
				{
					MaxPos = 0;
				}

				if(mVScrollBar.Value > MaxPos)
				{
					mVScrollBar.Value = MaxPos;
				}

				mVScrollBar.Enabled = VerticalDisplayableCount < mDocument.LineCount;
			}
		}

		/// <summary>
		/// Event handler for when the control is resizing.
		/// </summary>
		/// <param name="e">Information about the event.</param>
		protected override void OnResize(EventArgs e)
		{
			UpdateScrollbarPositions();
			UpdateScrollbars();

			base.OnResize(e);

			UpdateCaret();
			Invalidate();

			// NOTE: Important this is performed last
			if(!this.Size.IsEmpty)
			{
				mLastSize = this.Size;
			}
		}

		private void UpdateScrollbarPositions()
		{
			if(mToolStrip_Find.Visible)
			{
				mVScrollBar.Location = new Point(this.Width - mVScrollBar.Width, 0);
				mVScrollBar.Height = this.Height - mHScrollBar.Height - mToolStrip_Find.Height;

				mHScrollBar.Location = new Point(0, this.Height - mHScrollBar.Height - mToolStrip_Find.Height);
				mHScrollBar.Width = this.Width - mVScrollBar.Width;
			}
			else
			{
				mVScrollBar.Location = new Point(this.Width - mVScrollBar.Width, 0);
				mVScrollBar.Height = this.Height - mHScrollBar.Height;

				mHScrollBar.Location = new Point(0, this.Height - mHScrollBar.Height);
				mHScrollBar.Width = this.Width - mVScrollBar.Width;
			}
		}

		/// <summary>
		/// Event handler for when the vertical scrollbar has been modified.
		/// </summary>
		/// <param name="sender">Object that generated the event.</param>
		/// <param name="e">Information about the event.</param>
		private void mVScrollBar_ValueChanged(object sender, EventArgs e)
		{
			UpdateCaret();
			Invalidate();
		}

		/// <summary>
		/// Event handler for when the horizontal scrollbar has been modified.
		/// </summary>
		/// <param name="sender">Object that generated the event.</param>
		/// <param name="e">Information about the event.</param>
		private void mHScrollBar_ValueChanged(object sender, EventArgs e)
		{
			UpdateCaret();
			Invalidate();
		}

		/// <summary>
		/// Event handler for when the control has received focus.
		/// </summary>
		/// <param name="e">Information about the event.</param>
		protected override void OnGotFocus(EventArgs e)
		{
			base.OnGotFocus(e);
			InternalCreateCaret();
		}

		/// <summary>
		/// Event handler for when the control has lost focus.
		/// </summary>
		/// <param name="e">Information about the event.</param>
		protected override void OnLostFocus(EventArgs e)
		{
			base.OnLostFocus(e);
			InternalDestroyCaret();
			mIsSelecting = false;
			mScrollSelectedTextTimer.Enabled = false;
		}

		/// <summary>
		/// Creates the caret.
		/// </summary>
		private void InternalCreateCaret()
		{
			UpdateCaret();
		}

		/// <summary>
		/// Destroys the caret.
		/// </summary>
		private void InternalDestroyCaret()
		{
#if !__MonoCS__
			DestroyCaret();
#endif
		}

		/// <summary>
		/// Updates the position and view state of the caret.
		/// </summary>
		private void UpdateCaret()
		{
			// make sure the caret is within the bounds of the document
			if(mDocument == null)
			{
				mCaretPosition.Column = 0;
				mCaretPosition.Line = 0;
			}
			else
			{
				if(mDocument.LineCount <= mCaretPosition.Line)
				{
					mCaretPosition.Line = Math.Max(0, mDocument.LineCount - 1);
				}

				if(mDocument.GetLineLength(mCaretPosition.Line) < mCaretPosition.Column)
				{
					mCaretPosition.Column = mDocument.GetLineLength(mCaretPosition.Line);
				}
			}

			if(this.Focused)
			{
#if !__MonoCS__
				// only draw the caret if it's visible
				if(mCaretPosition.Line >= mVScrollBar.Value && mCaretPosition.Line <= mVScrollBar.Value + this.DisplayableVerticalLines
					&& mCaretPosition.Column >= mHScrollBar.Value && mCaretPosition.Column <= mHScrollBar.Value + this.DisplayableCharacters)
				{
					CreateCaret(this.Handle, IntPtr.Zero, 0, this.Font.Height);
					SetCaretPos((mCaretPosition.Column - mHScrollBar.Value) * mFontDimensions.Width, (mCaretPosition.Line - mVScrollBar.Value) * this.Font.Height);
					ShowCaret(this.Handle);
				}
				else
				{
					HideCaret(this.Handle);
				}
#endif
			}
		}

		/// <summary>
		/// Event handler for when a mouse button has been pressed.
		/// </summary>
		/// <param name="e">Information about the event.</param>
		protected override void OnMouseDown(MouseEventArgs e)
		{
			base.OnMouseDown(e);

			if((e.Button & MouseButtons.Left) == MouseButtons.Left || (e.Button & MouseButtons.Right) == MouseButtons.Right)
			{
				bool bHadSelection = this.HasSelectedText;
				bool bUpdateCaret = true;
				TextLocation CharacterPosition = GetCharacterPosFromCoord(e.X, e.Y);
				mIsSelecting = e.Button == MouseButtons.Left;

				if(!mIsSelecting)
				{
					int SelectionStart;
					int SelectionLength;

					if(GetSelectionForLine(CharacterPosition.Line, out SelectionStart, out SelectionLength) && CharacterPosition.Column >= SelectionStart && CharacterPosition.Column < SelectionStart + SelectionLength)
					{
						bUpdateCaret = false;
					}
				}

				if(bUpdateCaret && (mCaretPosition != CharacterPosition || bHadSelection))
				{
#if !__MonoCS__
					mCaretPosition = CharacterPosition;

					ushort KeyState = GetAsyncKeyState(VK_SHIFT);

					// if shift key is not pressed
					if((KeyState & 0x8000) == 0)
					{
						mSelectionStart = mCaretPosition;

						if(bHadSelection)
						{
							OnSelectionChanged(new EventArgs());
						}
					}
					else
					{
						// if shift is pressed then we must be changing the selection
						bHadSelection = true;
						OnSelectionChanged(new EventArgs());
					}
#endif
				}

				// NOTE: This is very important. If you do not set the ActiveControl to null then
				// child controls such as the "find" toolstrip container won't give focus back!
				// This means the caret may never come back and babies will cry.
				this.ActiveControl = null;
				UpdateCaret();

				if(bHadSelection)
				{
					Invalidate();
				}
			}
		}

		/// <summary>
		/// Event handler for when a mouse button has been released.
		/// </summary>
		/// <param name="e">Information about the event.</param>
		protected override void OnMouseUp(MouseEventArgs e)
		{
			base.OnMouseUp(e);

			mScrollSelectedTextTimer.Enabled = false;
			mIsSelecting = false;
		}

		/// <summary>
		/// Event handler for when the mouse cursor has been moved.
		/// </summary>
		/// <param name="e">Information about the event.</param>
		protected override void OnMouseMove(MouseEventArgs e)
		{
			base.OnMouseMove(e);

			if(mIsSelecting)
			{
				Rectangle TextArea = GetTextAreaRect();

				if(TextArea.Contains(e.Location))
				{
					mScrollSelectedTextTimer.Enabled = false;
					TextLocation NewCaretPos = GetCharacterPosFromCoord(e.X, e.Y);

					if(mCaretPosition != NewCaretPos)
					{
						mCaretPosition = NewCaretPos;
						UpdateCaret();
						Invalidate();
					}
				}
				else
				{
					mScrollSelectedTextTimer.Enabled = true;
				}
			}

			mLastMousePos = e.Location;
		}

		/// <summary>
		/// Generates a rectangle for the area within the client rectangle that's used for drawing text.
		/// </summary>
		/// <returns>The rectangle containing the area within the client rectangle that's used for drawing text.</returns>
		private Rectangle GetTextAreaRect()
		{
			Rectangle TextArea = this.ClientRectangle;
			TextArea.Width -= mVScrollBar.Width;
			TextArea.Height -= mHScrollBar.Height;

			if(mToolStrip_Find.Visible)
			{
				TextArea.Height -= mToolStrip_Find.Height;
			}

			return TextArea;
		}

		/// <summary>
		/// Event handler for when the mouse wheel has been moved.
		/// </summary>
		/// <param name="e">Information about the event.</param>
		protected override void OnMouseWheel(MouseEventArgs e)
		{
			base.OnMouseWheel(e);

			if(mDocument != null)
			{
				int LinesToScroll = SystemInformation.MouseWheelScrollLines * -(e.Delta / SystemInformation.MouseWheelScrollDelta);

				if(LinesToScroll < 0)
				{
					mVScrollBar.Value = Math.Max(0, mVScrollBar.Value + LinesToScroll);
				}
				else
				{
					mVScrollBar.Value = Math.Min(GetMaximumVScrollValue(), mVScrollBar.Value + LinesToScroll);
				}
			}
		}

		/// <summary>
		/// Retrieves the maximum value that the vertical scrollbar can be set to.
		/// </summary>
		/// <returns>The maximum value the vertical scrollbar can be set to.</returns>
		private int GetMaximumVScrollValue()
		{
			// add 1 to account for max - 1 (otherwise an extra line is displayed)
			return mVScrollBar.Maximum - mVScrollBar.LargeChange + 1;
		}

		/// <summary>
		/// Converts a screen coordinate into a character position.
		/// </summary>
		/// <param name="X">The X coordinate.</param>
		/// <param name="Y">The Y coordinate.</param>
		/// <returns>The location within the document of the screen coordinate.</returns>
		TextLocation GetCharacterPosFromCoord(int X, int Y)
		{
			if(mDocument == null || mDocument.LineCount == 0)
			{
				return new TextLocation(0, 0);
			}

			// remove negative coordinates
			Y = Math.Max(0, Y);
			X = Math.Max(0, X);

			int LineIndex = Y / this.Font.Height + mVScrollBar.Value;

			if(LineIndex >= mDocument.LineCount)
			{
				LineIndex = Math.Max(0, mDocument.LineCount - 1);
			}

			// If the mouse clicked in the right half of the character the caret will be moved to its right side, otherwise the left
			int ColumnIndex = (int)Math.Round((double)X / mFontDimensions.Width) + mHScrollBar.Value;
			int LineLength = mDocument.GetLineLength(LineIndex);

			if(ColumnIndex >= LineLength)
			{
				ColumnIndex = LineLength;
			}

			return new TextLocation(LineIndex, ColumnIndex);
		}

		/// <summary>
		/// Determines whether or not a line contains selected text.
		/// </summary>
		/// <param name="LineIndex">The index of the line to check.</param>
		/// <param name="SelectionStart">Receives the start location of the selected text - if any.</param>
		/// <param name="SelectionLength">Receives the length of the selected text on the current line - if any.</param>
		/// <returns>True if the line contains selected text.</returns>
		bool GetSelectionForLine(int LineIndex, out int SelectionStart, out int SelectionLength)
		{
			SelectionStart = 0;
			SelectionLength = 0;

			if(mDocument == null || mCaretPosition == mSelectionStart)
			{
				return false;
			}

			bool bRet = true;

			if(mCaretPosition.Line == mSelectionStart.Line && mCaretPosition.Line == LineIndex)
			{
				if(mCaretPosition.Column > mSelectionStart.Column)
				{
					SelectionStart = mSelectionStart.Column;
					SelectionLength = mCaretPosition.Column - mSelectionStart.Column;
				}
				else
				{
					SelectionStart = mCaretPosition.Column;
					SelectionLength = mSelectionStart.Column - mCaretPosition.Column;
				}
			}
			else if(mCaretPosition.Line == LineIndex)
			{
				if(mCaretPosition.Line > mSelectionStart.Line)
				{
					SelectionLength = mCaretPosition.Column;
				}
				else
				{
					SelectionStart = mCaretPosition.Column;
					SelectionLength = mDocument.GetLineLength(LineIndex) - SelectionStart;
				}
			}
			else if(mSelectionStart.Line == LineIndex)
			{
				if(mSelectionStart.Line > mCaretPosition.Line)
				{
					SelectionLength = mSelectionStart.Column;
				}
				else
				{
					SelectionStart = mSelectionStart.Column;
					SelectionLength = mDocument.GetLineLength(LineIndex) - SelectionStart;
				}
			}
			else if(LineIndex > Math.Min(mCaretPosition.Line, mSelectionStart.Line) && LineIndex < Math.Max(mCaretPosition.Line, mSelectionStart.Line))
			{
				SelectionLength = mDocument.GetLineLength(LineIndex);
			}
			else
			{
				bRet = false;
			}

			return bRet;
		}

		/// <summary>
		/// Event handler for when a key has been pressed.
		/// </summary>
		/// <param name="e">Information about the event.</param>
		protected override void OnKeyDown(KeyEventArgs e)
		{
			base.OnKeyDown(e);

			if(mDocument != null)
			{
				bool bHadSelection = (mCaretPosition != mSelectionStart);
				TextLocation OldCaretPosition = !bHadSelection ? mCaretPosition : mSelectionStart;
				bool bHandleShift = false;

				if((e.KeyCode == Keys.Insert || e.KeyCode == Keys.C) && e.Control)
				{
					CopySelectedText();
				}
				else if(e.KeyCode == Keys.A && e.Control)
				{
					e.Handled = true;

					SelectAll();
				}
				else if(e.KeyCode == Keys.F && e.Control)
				{
					EnterFindMode();
				}
				else if(e.KeyCode == Keys.F3)
				{
					if(e.Shift)
					{
						FindPrevious();
					}
					else
					{
						FindNext();
					}
				}
				else if(e.KeyCode == Keys.End)
				{
					if(e.Control)
					{
						mCaretPosition.Line = Math.Max(0, mDocument.LineCount - 1);
					}

					mCaretPosition.Column = mDocument.GetLineLength(mCaretPosition.Line);

					bHandleShift = true;

					ScrollToCaret();
				}
				else if(e.KeyCode == Keys.Home)
				{
					mCaretPosition.Column = 0;

					if(e.Control)
					{
						mCaretPosition.Line = 0;
					}

					bHandleShift = true;

					ScrollToCaret();
				}
				else if(e.KeyCode == Keys.Up)
				{
					MoveCaretUpLines(1, false);

					bHandleShift = true;

					ScrollToCaret();
				}
				else if(e.KeyCode == Keys.Down)
				{
					MoveCaretDownLines(1, false);

					bHandleShift = true;

					ScrollToCaret();
				}
				else if(e.KeyCode == Keys.Right)
				{
					++mCaretPosition.Column;

					if(mCaretPosition.Column >= mDocument.GetLineLength(mCaretPosition.Line))
					{
						++mCaretPosition.Line;

						if(mCaretPosition.Line >= mDocument.LineCount)
						{
							--mCaretPosition.Line;
							--mCaretPosition.Column;
						}
						else
						{
							mCaretPosition.Column = 0;
						}
					}

					bHandleShift = true;

					ScrollToCaret();
				}
				else if(e.KeyCode == Keys.Left)
				{
					--mCaretPosition.Column;

					if(mCaretPosition.Column < 0)
					{
						--mCaretPosition.Line;

						if(mCaretPosition.Line < 0)
						{
							mCaretPosition.Line = 0;
							mCaretPosition.Column = 0;
						}
						else
						{
							mCaretPosition.Column = mDocument.GetLineLength(mCaretPosition.Line);
						}
					}

					bHandleShift = true;
				}
				else if(e.KeyCode == Keys.PageUp)
				{
					mCaretPosition.Line = Math.Max(0, mCaretPosition.Line - mVScrollBar.LargeChange);

					int LineLength = mDocument.GetLineLength(mCaretPosition.Line);
					if(mCaretPosition.Column > LineLength)
					{
						mCaretPosition.Column = LineLength;
					}

					bHandleShift = true;
				}
				else if(e.KeyCode == Keys.PageDown)
				{
					mCaretPosition.Line = Math.Min(mDocument.LineCount - 1, mCaretPosition.Line + mVScrollBar.LargeChange);

					int LineLength = mDocument.GetLineLength(mCaretPosition.Line);
					if(mCaretPosition.Column > LineLength)
					{
						mCaretPosition.Column = LineLength;
					}

					bHandleShift = true;
				}

				if(bHandleShift)
				{
					// prevent the key from propagating to children
					e.Handled = true;

					if(e.Shift)
					{
						mSelectionStart = OldCaretPosition;
					}
					else
					{
						mSelectionStart = mCaretPosition;
					}

					if(mSelectionStart == mCaretPosition)
					{
						if(bHadSelection)
						{
							OnSelectionChanged(new EventArgs());
						}
					}
					else
					{
						OnSelectionChanged(new EventArgs());
					}

					ScrollToCaretOrInvalidate();
				}
			}
		}

		/// <summary>
		/// Moves the caret down in the document.
		/// </summary>
		/// <param name="NumLines">The number of lines to move the caret up.</param>
		/// <param name="bMoveSelectionStart">Pass true if the selection start is moved with the caret.</param>
		private void MoveCaretDownLines(int NumLines, bool bMoveSelectionStart)
		{
			if(NumLines < 0)
			{
				throw new ArgumentOutOfRangeException("NumLines", "NumLines must be greater than 0.");
			}

			if(mDocument == null)
			{
				return;
			}

			mCaretPosition.Line = Math.Min(mCaretPosition.Line + NumLines, Math.Max(0, mDocument.LineCount - 1));

			if(mCaretPosition.Column > mDocument.GetLineLength(mCaretPosition.Line))
			{
				mCaretPosition.Column = mDocument.GetLineLength(mCaretPosition.Line);
			}

			if(bMoveSelectionStart)
			{
				mSelectionStart = mCaretPosition;
			}
		}

		/// <summary>
		/// Moves the caret up in the document.
		/// </summary>
		/// <param name="NumLines">The number of lines to move the caret up.</param>
		/// <param name="bMoveSelectionStart">Pass true if the selection start is moved with the caret.</param>
		private void MoveCaretUpLines(int NumLines, bool bMoveSelectionStart)
		{
			if(NumLines < 0)
			{
				throw new ArgumentOutOfRangeException("NumLines", "NumLines must be greater than 0.");
			}

			if(mDocument == null)
			{
				return;
			}

			mCaretPosition.Line = Math.Max(mCaretPosition.Line - NumLines, 0);

			if(mCaretPosition.Column > mDocument.GetLineLength(mCaretPosition.Line))
			{
				mCaretPosition.Column = mDocument.GetLineLength(mCaretPosition.Line);
			}

			if(bMoveSelectionStart)
			{
				mSelectionStart = mCaretPosition;
			}
		}

		/// <summary>
		/// Copies the currently selected text to the clipboard.
		/// </summary>
		/// <returns>The currently selected text that was copied to the clipboard.</returns>
		public string CopySelectedText()
		{
			string Txt = this.SelectedText;

			if(Txt.Length > 0)
			{
				Clipboard.SetText(Txt, TextDataFormat.UnicodeText);
			}

			return Txt;
		}

		/// <summary>
		/// Selects all text and scrolls to the caret.
		/// </summary>
		public void SelectAll()
		{
			mSelectionStart = mDocument.BeginningOfDocument;
			mCaretPosition = mDocument.EndOfDocument;

			OnSelectionChanged(new EventArgs());

			if(!ScrollToCaret())
			{
				UpdateCaret();
				Invalidate();
			}
		}

		/// <summary>
		/// Scrolls the control so that the caret is visible.
		/// </summary>
		/// <returns>True if the control had to be scrolled.</returns>
		public bool ScrollToCaret()
		{
			bool bInvalidate = false;

			if(mCaretPosition.Line < mVScrollBar.Value)
			{
				mVScrollBar.Value = mCaretPosition.Line;
				bInvalidate = true;
			}
			else if(mCaretPosition.Line >= mVScrollBar.Value + this.DisplayableVerticalLines)
			{
				// since mVScrollBar.Value is an index if its equal to the total # of lines we still need to add 1 to move the scrollbar
				const int INDEX_PADDING = 1;

				mVScrollBar.Value = Math.Min(mVScrollBar.Maximum, mVScrollBar.Value + mCaretPosition.Line - (mVScrollBar.Value + this.DisplayableVerticalLines) + INDEX_PADDING);
				bInvalidate = true;
			}

			if(mCaretPosition.Column < mHScrollBar.Value)
			{
				mHScrollBar.Value = mCaretPosition.Column;
				bInvalidate = true;
			}
			else if(mCaretPosition.Column >= mHScrollBar.Value + this.DisplayableCharacters)
			{
				mHScrollBar.Value = Math.Min(mHScrollBar.Maximum, mHScrollBar.Value + mCaretPosition.Column - (mHScrollBar.Value + this.DisplayableCharacters));
				bInvalidate = true;
			}

			if(bInvalidate)
			{
				UpdateCaret();
				Invalidate();

				return true;
			}

			return false;
		}

		/// <summary>
		/// Allows us to handle the arrow keys.
		/// </summary>
		/// <param name="keyData">Information about the key to be checked.</param>
		/// <returns>True if the key is valid input.</returns>
		protected override bool IsInputKey(Keys keyData)
		{
			if((keyData & Keys.Left) == Keys.Left || (keyData & Keys.Right) == Keys.Right
				|| (keyData & Keys.Up) == Keys.Up || (keyData & Keys.Down) == Keys.Down)
			{
				return true;
			}

			return base.IsInputKey(keyData);
		}

		/// <summary>
		/// Determines whether a location within the current document is valid.
		/// </summary>
		/// <param name="Loc">The location to validate.</param>
		/// <returns>True if the supplied location is within the bounds of the document.</returns>
		bool IsValidTextLocation(TextLocation Loc)
		{
			bool bResult = false;

			if(mDocument != null)
			{
				bResult = mDocument.IsValidTextLocation(Loc);
			}

			return bResult;
		}

		/// <summary>
		/// Selects a range of text and brings the selection into view.
		/// </summary>
		/// <param name="StartLoc">The start location of the selection.</param>
		/// <param name="EndLoc">The end location (and caret location) of the selection.</param>
		public void Select(TextLocation StartLoc, TextLocation EndLoc)
		{
			if(mDocument != null)
			{
				if(!IsValidTextLocation(StartLoc))
				{
					throw new ArgumentOutOfRangeException("StartLoc");
				}

				if(!IsValidTextLocation(EndLoc))
				{
					throw new ArgumentOutOfRangeException("EndLoc");
				}

				mSelectionStart = StartLoc;
				mCaretPosition = EndLoc;

				OnSelectionChanged(new EventArgs());

				ScrollToCaretOrInvalidate();
			}
		}

		/// <summary>
		/// Selects a range of text and brings the selection into view.
		/// </summary>
		/// <param name="Range">The location of the selection and its length.</param>
		public void Select(FindResult Range)
		{
			if(mDocument != null)
			{
				TextLocation StartLoc = new TextLocation(Range.Line, Range.Column);

				if(!IsValidTextLocation(StartLoc))
				{
					throw new ArgumentOutOfRangeException("Range");
				}

				TextLocation EndLoc = new TextLocation(Range.Line, Math.Min(Range.Column + Range.Length, mDocument.GetLineLength(Range.Line)));

				Select(StartLoc, EndLoc);
			}
		}

		/// <summary>
		/// Finds the first occurence of the specified string within the document if it exists.
		/// </summary>
		/// <param name="Txt">The string to find.</param>
		/// <param name="StartLoc">The position within the document to begin searching.</param>
		/// <param name="EndLoc">The position within the document to end searching.</param>
		/// <param name="Flags">Flags telling the document how to conduct its search.</param>
		/// <param name="Result">The location within the document of the supplied text.</param>
		/// <returns>True if the text was found.</returns>
		public bool Find(string Txt, TextLocation StartLoc, TextLocation EndLoc, RichTextBoxFinds Flags, out FindResult Result)
		{
			bool bResult = false;

			if(mDocument == null)
			{
				Result = FindResult.Empty;
			}
			else
			{
				bResult = mDocument.Find(Txt, StartLoc, EndLoc, Flags, out Result);
			}

			if(bResult && (Flags & RichTextBoxFinds.NoHighlight) != RichTextBoxFinds.NoHighlight)
			{
				Select(Result);
			}

			return bResult;
		}

		/// <summary>
		/// Clears the text in the source document.
		/// </summary>
		public void Clear()
		{
			if(mDocument != null)
			{
				mDocument.Clear();
			}
		}

		/// <summary>
		/// Saves the source document to a file.
		/// </summary>
		/// <param name="Name">The name of the file to write the document to.</param>
		public void SaveToFile(string Name)
		{
			if(mDocument != null)
			{
				mDocument.SaveToFile(Name);
			}
		}

		/// <summary>
		/// Scrolls the caret to the end of the document.
		/// </summary>
		public void ScrollToEnd()
		{
			if(mDocument != null)
			{
				bool bHadSelection = this.HasSelectedText;

				mSelectionStart = mCaretPosition = mDocument.EndOfDocument;

				if(bHadSelection)
				{
					OnSelectionChanged(new EventArgs());
				}

				ScrollToCaretOrInvalidate();
			}
		}

		/// <summary>
		/// Scrolls the caret to the beginning of the document.
		/// </summary>
		public void ScrollToHome()
		{
			if(mDocument != null)
			{
				bool bHadSelection = this.HasSelectedText;

				mSelectionStart = mCaretPosition = mDocument.BeginningOfDocument;

				if(bHadSelection)
				{
					OnSelectionChanged(new EventArgs());
				}

				ScrollToCaretOrInvalidate();
			}
		}

		/// <summary>
		/// Scrolls to the location of the caret. If the caret is already in view then it forces a redraw.
		/// </summary>
		private void ScrollToCaretOrInvalidate()
		{
			if(!ScrollToCaret())
			{
				UpdateCaret();
				Invalidate();
			}
		}

		/// <summary>
		/// Event handler for copying text.
		/// </summary>
		/// <param name="sender">The object that initiated the event.</param>
		/// <param name="e">Additional information about the event.</param>
		private void mCtxDefault_Copy_Click(object sender, EventArgs e)
		{
			CopySelectedText();
		}

		/// <summary>
		/// Event handler for clearing the document.
		/// </summary>
		/// <param name="sender">The object that initiated the event.</param>
		/// <param name="e">Additional information about the event.</param>
		private void ToolStripMenuItem_Clear_Click( object sender, EventArgs e )
		{
			if( mDocument != null )
			{
				mDocument.Clear();
			}
		}

		/// <summary>
		/// Event handler for scrolling to the end of the document.
		/// </summary>
		/// <param name="sender">The object that initiated the event.</param>
		/// <param name="e">Additional information about the event.</param>
		private void mCtxDefault_ScrollToEnd_Click(object sender, EventArgs e)
		{
			ScrollToEnd();
		}

		/// <summary>
		/// Event handler for scrolling to the beginning of the document.
		/// </summary>
		/// <param name="sender">The object that initiated the event.</param>
		/// <param name="e">Additional information about the event.</param>
		private void mCtxDefault_ScrollToHome_Click(object sender, EventArgs e)
		{
			ScrollToHome();
		}

		/// <summary>
		/// Event handler for auto-scrolling selected text.
		/// </summary>
		/// <param name="sender">The object that initiated the event.</param>
		/// <param name="e">Additional information about the event.</param>
		private void mScrollSelectedTextTimer_Tick(object sender, EventArgs e)
		{
			if(mDocument == null)
			{
				return;
			}

			TextLocation NewCaretPos = GetCharacterPosFromCoord(mLastMousePos.X, mLastMousePos.Y);
			Rectangle TextArea = GetTextAreaRect();
			bool bRequiresRedraw = true;

			if(mLastMousePos.Y >= TextArea.Bottom)
			{
				mCaretPosition.Line = Math.Min(mDocument.LineCount - 1, mVScrollBar.Value + this.DisplayableVerticalLines - 1);
				MoveCaretDownLines(2, false);
			}
			else if(mLastMousePos.Y <= TextArea.Top)
			{
				mCaretPosition.Line = mVScrollBar.Value;
				MoveCaretUpLines(2, false);
			}
			else
			{
				bRequiresRedraw = mCaretPosition.Line != NewCaretPos.Line;
				mCaretPosition.Line = NewCaretPos.Line;
			}

			int NewColumn = 0;

			// now that we have an updated line position get an updated column position
			NewCaretPos = GetCharacterPosFromCoord(mLastMousePos.X, mLastMousePos.Y);

			if(mLastMousePos.X >= TextArea.Right)
			{
				// NOTE: mCaretPosition.Line has already been updated so don't use NewCaretPos.Line
				NewColumn = Math.Min(mDocument.GetLineLength(mCaretPosition.Line), mHScrollBar.Value + this.DisplayableCharacters + 2);
			}
			else if(mLastMousePos.X <= TextArea.Left)
			{
				NewColumn = Math.Max(0, mHScrollBar.Value - 2);
			}
			else
			{
				NewColumn = NewCaretPos.Column;
			}


			if(!bRequiresRedraw)
			{
				bRequiresRedraw = mCaretPosition.Column != NewColumn;
			}

			mCaretPosition.Column = NewColumn;

			if(!ScrollToCaret() && bRequiresRedraw)
			{
				UpdateCaret();
				Invalidate();
			}
		}

		/// <summary>
		/// Event handler for when the document has been modified.
		/// </summary>
		/// <param name="sender">The object that initiated the event.</param>
		/// <param name="e">Additional information about the event.</param>
		void mDocument_Modified(object sender, EventArgs e)
		{
			TextLocation PrevDocSize = mCurrentDocumentSize;
			mCurrentDocumentSize.Line = mDocument.LineCount;
			mCurrentDocumentSize.Column = mDocument.GetLineLength(mCurrentDocumentSize.Line - 1);

			bool bInvalidate = true;
			bool bHadSelection = this.HasSelectedText;

			if(mCurrentDocumentSize > PrevDocSize)
			{
				// we have to do these checks before we update he scrollbars
				if(mAutoScroll && !bHadSelection && mCaretPosition.Column == PrevDocSize.Column && mCaretPosition.Line == Math.Max(0, PrevDocSize.Line - 1)
					&& mVScrollBar.Value == Math.Max(0, GetMaximumVScrollValue()))
				{
					// ScrollToCaret() is dependent on scrollbar location so update that now instead of later
					UpdateScrollbars();

					mCaretPosition.Line = Math.Max(0, mDocument.LineCount - 1);
					mCaretPosition.Column = mDocument.GetLineLength(mCaretPosition.Line);
					mSelectionStart = mCaretPosition;

					bInvalidate = !ScrollToCaret();
				}
				else
				{
					UpdateScrollbars();
				}
			}
			else
			{
				// if doc size shrunk we need to make sure the caret is still within range
				if(mCaretPosition > mCurrentDocumentSize)
				{
					mCaretPosition = mCurrentDocumentSize;

					if(bHadSelection)
					{
						if(mCaretPosition > mSelectionStart)
						{
							mSelectionStart = mCaretPosition;
						}

						OnSelectionChanged(new EventArgs());
					}
					else
					{
						mSelectionStart = mCaretPosition;
					}
				}
				else if(bHadSelection && mSelectionStart > mCurrentDocumentSize)
				{
					mSelectionStart = mCurrentDocumentSize;
					OnSelectionChanged(new EventArgs());
				}

				UpdateScrollbars();
			}

			if(bInvalidate)
			{
				UpdateCaret();
				Invalidate();
			}
		}

		/// <summary>
		/// Event handler for when the visibility of the "Find" toolstrip changes.
		/// </summary>
		/// <param name="sender">The object that initiated the event.</param>
		/// <param name="e">Additional information about the event.</param>
		private void mToolStrip_Find_VisibleChanged(object sender, EventArgs e)
		{
			UpdateScrollbarPositions();
			UpdateScrollbars();
			UpdateCaret();
			Invalidate();
		}

		/// <summary>
		/// Event handler for when the mouse movers over the control.
		/// </summary>
		/// <param name="sender">The object that initiated the event.</param>
		/// <param name="e">Additional information about the event.</param>
		private void OutputWindowView_MouseMove(object sender, MouseEventArgs e)
		{
			Control Child = GetChildAtPoint(e.Location, GetChildAtPointSkip.Invisible);
			Rectangle ScrollSpace = new Rectangle(mHScrollBar.Width, mHScrollBar.Location.Y, mVScrollBar.Width, mHScrollBar.Height);

			if(ScrollSpace.Contains(e.Location))
			{
				if(this.Cursor != Cursors.Arrow)
				{
					this.Cursor = Cursors.Arrow;
				}
			}
			else if(Child != null && this.Cursor != Cursors.Arrow)
			{
				this.Cursor = Cursors.Arrow;
			}
			else if(Child == null && this.Cursor != Cursors.IBeam)
			{
				this.Cursor = Cursors.IBeam;
			}
		}

		/// <summary>
		/// Event handler for when the mouse movers over a child control.
		/// </summary>
		/// <param name="sender">The object that initiated the event.</param>
		/// <param name="e">Additional information about the event.</param>
		private void ChildControl_MouseEnter(object sender, EventArgs e)
		{
			if(this.Cursor != Cursors.Arrow)
			{
				this.Cursor = Cursors.Arrow;
			}
		}

		/// <summary>
		/// Tells the view to display the "find" tool strip.
		/// </summary>
		public void EnterFindMode()
		{
			if(this.HasSelectedText)
			{
				mToolStripTextBox_Find.Text = this.SelectedText;
			}

			mToolStrip_Find.Visible = true;
			mToolStripTextBox_Find.Focus();
		}

		/// <summary>
		/// Tells the view to close the "find" tool strip.
		/// </summary>
		public void ExitFindMode()
		{
			mToolStrip_Find.Visible = false;
		}

		/// <summary>
		/// Searches down within the text box for the specified search string.
		/// </summary>
		public void FindNext()
		{
			if(!this.IsInFindMode)
			{
				EnterFindMode();

				// Give focus back to our edit control so we can perform the find
				this.ActiveControl = null;
			}

			FindNext(null);
		}

		/// <summary>
		/// Searches down within the text box for the specified search string.
		/// </summary>
		/// <param name="DownSearchStart">The location within the document to begin searching.</param>
		void FindNext(TextLocation? DownSearchStart)
		{
			if(this.mDocument != null)
			{
				FindResult Result;
				RichTextBoxFinds Flags = mCheckBox_MatchCase.Checked ? RichTextBoxFinds.MatchCase : RichTextBoxFinds.None;

				if(!DownSearchStart.HasValue)
				{
					DownSearchStart = this.SelectionStart >= this.SelectionEnd ? this.SelectionStart : this.SelectionEnd;
				}

				bool bFound = true;
				string TxtToSearchFor = mToolStripTextBox_Find.Text.Trim();

				if(!this.Find(TxtToSearchFor, DownSearchStart.Value, this.mDocument.EndOfDocument, Flags, out Result))
				{
					if(!this.Find(TxtToSearchFor, this.mDocument.BeginningOfDocument, new TextLocation(DownSearchStart.Value.Line, mDocument.GetLineLength(DownSearchStart.Value.Line)), Flags, out Result))
					{
						//MessageBox.Show(this, "The specified search string does not exist!", mToolStripTextBox_Find.Text);
						bFound = false;
					}
				}

				UpdateFindBoxColor(bFound);
			}
		}

		/// <summary>
		/// Searches up within the text box for the specified search string.
		/// </summary>
		public void FindPrevious()
		{
			if(!this.IsInFindMode)
			{
				EnterFindMode();

				// Give focus back to our edit control so we can perform the find
				this.ActiveControl = null;
			}

			FindPrevious(null);
		}

		/// <summary>
		/// Searches up within the text box for the specified search string.
		/// </summary>
		/// <param name="UpSearchStart">The location within the document to begin searching.
		void FindPrevious(TextLocation? UpSearchStart)
		{
			if(this.mDocument != null)
			{
				FindResult Result;
				RichTextBoxFinds Flags = (mCheckBox_MatchCase.Checked ? RichTextBoxFinds.MatchCase : RichTextBoxFinds.None) | RichTextBoxFinds.Reverse;

				if(!UpSearchStart.HasValue)
				{
					UpSearchStart = this.SelectionStart <= this.SelectionEnd ? this.SelectionStart : this.SelectionEnd;
				}

				bool bFound = true;
				string TxtToSearchFor = mToolStripTextBox_Find.Text.Trim();

				if(!this.Find(TxtToSearchFor, UpSearchStart.Value, this.mDocument.BeginningOfDocument, Flags, out Result))
				{
					if(!this.Find(TxtToSearchFor, this.mDocument.EndOfDocument, new TextLocation(UpSearchStart.Value.Line, 0), Flags, out Result))
					{
						bFound = false;
					}
				}

				UpdateFindBoxColor(bFound);
			}
		}

		/// <summary>
		/// Updates the color of the "Find" text box.
		/// </summary>
		/// <param name="bFound">True if the text in the text box is valid.</param>
		private void UpdateFindBoxColor(bool bFound)
		{
			if(bFound)
			{
				if(mToolStripTextBox_Find.BackColor != SystemColors.Window)
				{
					mToolStripTextBox_Find.BackColor = SystemColors.Window;
					mToolStripTextBox_Find.ForeColor = SystemColors.ControlText;
				}
			}
			else
			{
				if(mToolStripTextBox_Find.BackColor != INVALID_SEARCH_BACKCOLOR)
				{
					mToolStripTextBox_Find.BackColor = INVALID_SEARCH_BACKCOLOR;
					mToolStripTextBox_Find.ForeColor = INVALID_SEARCH_FORECOLOR;

					// We need to force a redraw if the combobox is just switching to an invalid state as it's possible
					// that "find" search items were highlighted and they need to be redrawn
					Invalidate();
				}

				System.Media.SystemSounds.Beep.Play();
			}
		}

		/// <summary>
		/// Event handler for when the "Find Next" button has been clicked.
		/// </summary>
		/// <param name="sender">The object that initiated the event.</param>
		/// <param name="e">Additional information about the event.</param>
		private void mToolStripButton_FindNext_Click(object sender, EventArgs e)
		{
			FindNext();
		}

		/// <summary>
		/// Event handler for when the "Find Previous" button has been clicked.
		/// </summary>
		/// <param name="sender">The object that initiated the event.</param>
		/// <param name="e">Additional information about the event.</param>
		private void mToolStripButton_FindPrevious_Click(object sender, EventArgs e)
		{
			FindPrevious();
		}

		/// <summary>
		/// Event handler or when the text in the "find" text box has changed or the state of the "match case" checkbox has changed.
		/// </summary>
		/// <param name="sender">The object that initiated the event.</param>
		/// <param name="e">Additional information about the event.</param>
		private void mToolStripTextBox_Find_TextChanged(object sender, EventArgs e)
		{
			if(mToolStripTextBox_Find.Text.Trim().Length == 0)
			{
				mCaretPosition = mSelectionStart;
				UpdateFindBoxColor(true);
				UpdateCaret();
				Invalidate();
			}
			else
			{
				TextLocation StartLocation = mCaretPosition <= mSelectionStart ? mCaretPosition : mSelectionStart;
				FindNext(StartLocation);
			}
		}
	}
}
