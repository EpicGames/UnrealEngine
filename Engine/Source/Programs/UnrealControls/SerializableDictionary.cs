// Copyright Epic Games, Inc. All Rights Reserved.


using System;
using System.Collections.Generic;
using System.Text;
using System.Xml.Serialization;
using System.Runtime.Serialization;
using System.Collections;

namespace UnrealControls
{
	[Serializable]
	public struct SerializableKeyValuePair<TKey, TValue>
	{
		TKey mKey;
		TValue mValue;

		public TKey Key
		{
			get { return mKey; }
			set { mKey = value; }
		}

		public TValue Value
		{
			get { return mValue; }
			set { mValue = value; }
		}

		public SerializableKeyValuePair(KeyValuePair<TKey, TValue> Pair)
		{
			mKey = Pair.Key;
			mValue = Pair.Value;
		}
	}

	[Serializable]
	public class SerializableDictionary<TKey, TValue> : ICollection<SerializableKeyValuePair<TKey, TValue>>, IEnumerable<SerializableKeyValuePair<TKey, TValue>>, IEnumerable
	{
		Dictionary<TKey, TValue> mInternalDictionary = new Dictionary<TKey, TValue>();

		public SerializableDictionary()
		{
		}

		protected SerializableDictionary(SerializationInfo si, StreamingContext context)
		{
			IDictionary<TKey, TValue> iDic = mInternalDictionary;
			List<SerializableKeyValuePair<TKey, TValue>> Data = si.GetValue("Data", typeof(List<SerializableKeyValuePair<TKey, TValue>>)) as List<SerializableKeyValuePair<TKey, TValue>>;

			foreach(SerializableKeyValuePair<TKey, TValue> CurPair in Data)
			{
				iDic.Add(CurPair.Key, CurPair.Value);
			}
		}

		#region IDictionary<TKey,TValue> Members

		public void Add(TKey key, TValue value)
		{
			mInternalDictionary.Add(key, value);
		}

		public bool ContainsKey(TKey key)
		{
			return mInternalDictionary.ContainsKey(key);
		}

		[XmlIgnore]
		public ICollection<TKey> Keys
		{
			get { return mInternalDictionary.Keys; }
		}

		public bool Remove(TKey key)
		{
			return mInternalDictionary.Remove(key);
		}

		public bool TryGetValue(TKey key, out TValue value)
		{
			return mInternalDictionary.TryGetValue(key, out value);
		}

		[XmlIgnore]
		public ICollection<TValue> Values
		{
			get { return mInternalDictionary.Values; }
		}

		[XmlIgnore]
		public TValue this[TKey key]
		{
			get
			{
				return mInternalDictionary[key];
			}
			set
			{
				mInternalDictionary[key] = value;
			}
		}

		#endregion

		#region ICollection<KeyValuePair<TKey,TValue>> Members

		public void Add(SerializableKeyValuePair<TKey, TValue> item)
		{
			IDictionary<TKey, TValue> iDic = mInternalDictionary;
			iDic.Add(new KeyValuePair<TKey, TValue>(item.Key, item.Value));
		}

		public void Clear()
		{
			mInternalDictionary.Clear();
		}

		public bool Contains(SerializableKeyValuePair<TKey, TValue> item)
		{
			IDictionary<TKey, TValue> iDic = mInternalDictionary;
			return iDic.Contains(new KeyValuePair<TKey, TValue>(item.Key, item.Value));
		}

		public void CopyTo(SerializableKeyValuePair<TKey, TValue>[] array, int arrayIndex)
		{
			foreach(KeyValuePair<TKey, TValue> CurPair in mInternalDictionary)
			{
				array[arrayIndex] = new SerializableKeyValuePair<TKey, TValue>(CurPair);
				++arrayIndex;
			}
		}

		[XmlIgnore]
		public int Count
		{
			get { return mInternalDictionary.Count; }
		}

		[XmlIgnore]
		public bool IsReadOnly
		{
			get { IDictionary<TKey, TValue> iDic = mInternalDictionary; return iDic.IsReadOnly; }
		}

		public bool Remove(SerializableKeyValuePair<TKey, TValue> item)
		{
			IDictionary<TKey, TValue> iDic = mInternalDictionary;
			return iDic.Remove(new KeyValuePair<TKey, TValue>(item.Key, item.Value));
		}

		#endregion

		#region IEnumerable<KeyValuePair<TKey,TValue>> Members

		public IEnumerator<SerializableKeyValuePair<TKey, TValue>> GetEnumerator()
		{
			foreach(KeyValuePair<TKey, TValue> CurPair in mInternalDictionary)
			{
				yield return new SerializableKeyValuePair<TKey, TValue>(CurPair);
			}
		}

		#endregion

		#region IEnumerable Members

		System.Collections.IEnumerator System.Collections.IEnumerable.GetEnumerator()
		{
			foreach(KeyValuePair<TKey, TValue> CurPair in mInternalDictionary)
			{
				yield return new SerializableKeyValuePair<TKey, TValue>(CurPair);
			}
		}

		#endregion
	}
}
