// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;

namespace UnrealControls
{
	/// <summary>
	/// Represents a location within the text of a document.
	/// </summary>
	public struct TextLocation
	{
		private int mLine;
		private int mColumn;

		/// <summary>
		/// Gets/Sets the line number.
		/// </summary>
		public int Line
		{
			get { return mLine; }
			set { mLine = value; }
		}

		/// <summary>
		/// Gets/Sets the character column.
		/// </summary>
		public int Column
		{
			get { return mColumn; }
			set { mColumn = value; }
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="Line">The line number.</param>
		/// <param name="Column">The character column.</param>
		public TextLocation(int Line, int Column)
		{
			this.mLine = Line;
			this.mColumn = Column;
		}

		/// <summary>
		/// Operator overload for !=.
		/// </summary>
		/// <param name="Left">Left operand.</param>
		/// <param name="Right">Right operand.</param>
		/// <returns>True if the left operand is not equal to the right operand.</returns>
		public static bool operator !=(TextLocation Left, TextLocation Right)
		{
			return Left.Line != Right.Line || Left.Column != Right.Column;
		}

		/// <summary>
		/// Operator overload for ==.
		/// </summary>
		/// <param name="Left">Left operand.</param>
		/// <param name="Right">Right operand.</param>
		/// <returns>True if the left operand is equal to the right operand.</returns>
		public static bool operator ==(TextLocation Left, TextLocation Right)
		{
			return Left.Line == Right.Line && Left.Column == Right.Column;
		}

		/// <summary>
		/// Operator overload for <.
		/// </summary>
		/// <param name="Left">Left operand.</param>
		/// <param name="Right">Right operand.</param>
		/// <returns>True if the left operand is less than the right operand.</returns>
		public static bool operator <(TextLocation Left, TextLocation Right)
		{
			if(Left.Line == Right.Line)
			{
				return Left.Column < Right.Column;
			}

			return Left.Line < Right.Line;
		}

		/// <summary>
		/// Operator overload for >.
		/// </summary>
		/// <param name="Left">Left operand.</param>
		/// <param name="Right">Right operand.</param>
		/// <returns>True if the left operand is greater than the right operand.</returns>
		public static bool operator >(TextLocation Left, TextLocation Right)
		{
			if(Left.Line == Right.Line)
			{
				return Left.Column > Right.Column;
			}

			return Left.Line > Right.Line;
		}

		/// <summary>
		/// Operator overload for <=.
		/// </summary>
		/// <param name="Left">Left operand.</param>
		/// <param name="Right">Right operand.</param>
		/// <returns>True if the left operand is less than or equal to the right operand.</returns>
		public static bool operator <=(TextLocation Left, TextLocation Right)
		{
			if(Left == Right)
			{
				return true;
			}

			if(Left.Line == Right.Line)
			{
				return Left.Column < Right.Column;
			}

			return Left.Line < Right.Line;
		}

		/// <summary>
		/// Operator overload for >=.
		/// </summary>
		/// <param name="Left">Left operand.</param>
		/// <param name="Right">Right operand.</param>
		/// <returns>True if the left operand is greater than or equal to the right operand.</returns>
		public static bool operator >=(TextLocation Left, TextLocation Right)
		{
			if(Left == Right)
			{
				return true;
			}

			if(Left.Line == Right.Line)
			{
				return Left.Column > Right.Column;
			}

			return Left.Line > Right.Line;
		}

		/// <summary>
		/// Checks an object for equality.
		/// </summary>
		/// <param name="obj">The object to check.</param>
		/// <returns>True if the object is equal to the one conducting the check.</returns>
		public override bool Equals(object obj)
		{
			if(obj is TextLocation)
			{
				TextLocation Marker = (TextLocation)obj;

				return this.mLine == Marker.mLine && this.mColumn == Marker.mColumn;
			}

			return base.Equals(obj);
		}

		/// <summary>
		/// Generates a hash code.
		/// </summary>
		/// <returns>A hash code for the object.</returns>
		public override int GetHashCode()
		{
			return base.GetHashCode();
		}
	}
}
