// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Data;
using System.Drawing;
using System.IO;
using System.Text;
using System.Windows.Forms;
using System.Reflection;

namespace UnrealControls
{
	/// <summary>
	/// This class represents an about box for an Unreal application.
	/// </summary>
	public partial class UnrealAboutBox : Form
	{
		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="AppIcon">The icon of the host application.</param>
		public UnrealAboutBox(Icon AppIcon, EngineVersioning Versioning)
		{
			InitializeComponent();

			Assembly EntryAssembly = Assembly.GetEntryAssembly();
			
			mLabel_AppVersion.Text = mLabel_AppVersion.Text.Replace("N/A", EntryAssembly.GetName().Version.ToString());

			if( Versioning == null )
			{
				Versioning = new EngineVersioning();
			}

			if( Versioning.Changelist > -1 )
			{
				mLabel_Changelist.Text = mLabel_Changelist.Text.Replace( "N/A", Versioning.Changelist.ToString() );
			}

			if( Versioning.EngineVersion > -1 )
			{
				mLabel_EngineVersion.Text = mLabel_EngineVersion.Text.Replace( "N/A", Versioning.EngineVersion.ToString() );
			}

			mPictureBox_AppIcon.Image = AppIcon.ToBitmap();
		}
	}
}