<?xml version="1.0" encoding="utf-8"?>
<Project DefaultTargets="Build" xmlns="http://schemas.microsoft.com/developer/msbuild/2003" ToolsVersion="12.0">
  <PropertyGroup>
    <Configuration Condition=" '$(Configuration)' == '' ">Debug</Configuration>
    <Platform Condition=" '$(Platform)' == '' ">AnyCPU</Platform>
    <ProductVersion>9.0.30729</ProductVersion>
    <SchemaVersion>2.0</SchemaVersion>
    <ProjectGuid>{8F079097-3724-4B31-AD3B-0A5B76C26376}</ProjectGuid>
    <OutputType>Library</OutputType>
    <AppDesignerFolder>Properties</AppDesignerFolder>
    <RootNamespace>UnrealControls</RootNamespace>
    <AssemblyName>UnrealControls</AssemblyName>
    <SccProjectName>SAK</SccProjectName>
    <SccLocalPath>SAK</SccLocalPath>
    <SccAuxPath>SAK</SccAuxPath>
    <SccProvider>SAK</SccProvider>
    <FileUpgradeFlags>
    </FileUpgradeFlags>
    <OldToolsVersion>3.5</OldToolsVersion>
    <UpgradeBackupLocation>
    </UpgradeBackupLocation>
    <TargetFrameworkVersion>v4.6.2</TargetFrameworkVersion>
    <IsWebBootstrapper>false</IsWebBootstrapper>
    <TargetFrameworkProfile>Client</TargetFrameworkProfile>
    <PublishUrl>publish\</PublishUrl>
    <Install>true</Install>
    <InstallFrom>Disk</InstallFrom>
    <UpdateEnabled>false</UpdateEnabled>
    <UpdateMode>Foreground</UpdateMode>
    <UpdateInterval>7</UpdateInterval>
    <UpdateIntervalUnits>Days</UpdateIntervalUnits>
    <UpdatePeriodically>false</UpdatePeriodically>
    <UpdateRequired>false</UpdateRequired>
    <MapFileExtensions>true</MapFileExtensions>
    <ApplicationRevision>0</ApplicationRevision>
    <ApplicationVersion>1.0.0.%2a</ApplicationVersion>
    <UseApplicationTrust>false</UseApplicationTrust>
    <BootstrapperEnabled>true</BootstrapperEnabled>
    <TargetFrameworkProfile>
    </TargetFrameworkProfile>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Debug|AnyCPU' ">
    <DebugSymbols>true</DebugSymbols>
    <DebugType>full</DebugType>
    <Optimize>false</Optimize>
    <OutputPath>..\..\..\Binaries\DotNET\</OutputPath>
    <DefineConstants>DEBUG;TRACE</DefineConstants>
    <ErrorReport>prompt</ErrorReport>
    <WarningLevel>4</WarningLevel>
    <AllowUnsafeBlocks>true</AllowUnsafeBlocks>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Development|AnyCPU' ">
    <DebugType>pdbonly</DebugType>
    <Optimize>true</Optimize>
    <OutputPath>..\..\..\Binaries\DotNET\</OutputPath>
    <DefineConstants>TRACE</DefineConstants>
    <ErrorReport>prompt</ErrorReport>
    <WarningLevel>4</WarningLevel>
    <AllowUnsafeBlocks>true</AllowUnsafeBlocks>
  </PropertyGroup>
  <ItemGroup>
    <Reference Include="System" />
    <Reference Include="System.Core">
      <RequiredTargetFramework>3.5</RequiredTargetFramework>
    </Reference>
    <Reference Include="System.Data" />
    <Reference Include="System.Drawing" />
    <Reference Include="System.Windows.Forms" />
    <Reference Include="System.Xml" />
  </ItemGroup>
  <ItemGroup>
    <Compile Include="..\Shared\MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
    <Compile Include="ColoredDocumentLineSegment.cs" />
    <Compile Include="ColoredTextRange.cs" />
    <Compile Include="ColorPair.cs" />
    <Compile Include="DynamicTabControl.cs">
      <SubType>UserControl</SubType>
    </Compile>
    <Compile Include="DynamicTabControl.designer.cs">
      <DependentUpon>DynamicTabControl.cs</DependentUpon>
    </Compile>
    <Compile Include="DynamicTabControlMenuItem.cs">
      <SubType>Component</SubType>
    </Compile>
    <Compile Include="DynamicTabPage.cs">
      <SubType>UserControl</SubType>
    </Compile>
    <Compile Include="EngineVersioning.cs" />
    <Compile Include="FindResult.cs" />
    <Compile Include="GameLocator.cs" />
    <Compile Include="OutputWindowDocumentLineAddedEventArgs.cs" />
    <Compile Include="MenuArrow.cs">
      <SubType>UserControl</SubType>
    </Compile>
    <Compile Include="MenuArrow.designer.cs">
      <DependentUpon>MenuArrow.cs</DependentUpon>
    </Compile>
    <Compile Include="OutputWindowDocument.cs" />
    <Compile Include="OutputWindowView.cs">
      <SubType>UserControl</SubType>
    </Compile>
    <Compile Include="OutputWindowView.Designer.cs">
      <DependentUpon>OutputWindowView.cs</DependentUpon>
    </Compile>
    <Compile Include="Properties\AssemblyInfo.cs" />
    <Compile Include="Properties\Resources.Designer.cs">
      <AutoGen>True</AutoGen>
      <DesignTime>True</DesignTime>
      <DependentUpon>Resources.resx</DependentUpon>
    </Compile>
    <Compile Include="SerializableDictionary.cs" />
    <Compile Include="TextLocation.cs" />
    <Compile Include="UnrealAboutBox.cs">
      <SubType>Form</SubType>
    </Compile>
    <Compile Include="UnrealAboutBox.Designer.cs">
      <DependentUpon>UnrealAboutBox.cs</DependentUpon>
    </Compile>
    <Compile Include="UnrealToolStripItemRenderer.cs">
    </Compile>
    <Compile Include="Utilities.cs" />
    <Compile Include="XButton.cs">
      <SubType>UserControl</SubType>
    </Compile>
    <Compile Include="XButton.designer.cs">
      <DependentUpon>XButton.cs</DependentUpon>
    </Compile>
  </ItemGroup>
  <ItemGroup>
    <EmbeddedResource Include="OutputWindowView.resx">
      <DependentUpon>OutputWindowView.cs</DependentUpon>
      <SubType>Designer</SubType>
    </EmbeddedResource>
    <EmbeddedResource Include="Properties\Resources.resx">
      <SubType>Designer</SubType>
      <Generator>ResXFileCodeGenerator</Generator>
      <LastGenOutput>Resources.Designer.cs</LastGenOutput>
    </EmbeddedResource>
    <EmbeddedResource Include="UnrealAboutBox.resx">
      <SubType>Designer</SubType>
      <DependentUpon>UnrealAboutBox.cs</DependentUpon>
    </EmbeddedResource>
  </ItemGroup>
  <ItemGroup>
    <EmbeddedResource Include="Resources\XButton.bmp" />
  </ItemGroup>
  <ItemGroup>
    <EmbeddedResource Include="Resources\ArrowDown.bmp" />
    <EmbeddedResource Include="Resources\ArrowUp.bmp" />
  </ItemGroup>
  <ItemGroup>
    <BootstrapperPackage Include="Microsoft.Net.Client.3.5">
      <Visible>False</Visible>
      <ProductName>.NET Framework 3.5 SP1 Client Profile</ProductName>
      <Install>false</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Net.Framework.2.0">
      <Visible>False</Visible>
      <ProductName>.NET Framework 2.0 %28x86%29</ProductName>
      <Install>true</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Net.Framework.3.0">
      <Visible>False</Visible>
      <ProductName>.NET Framework 3.0 %28x86%29</ProductName>
      <Install>false</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Net.Framework.3.5">
      <Visible>False</Visible>
      <ProductName>.NET Framework 3.5</ProductName>
      <Install>false</Install>
    </BootstrapperPackage>
    <BootstrapperPackage Include="Microsoft.Net.Framework.3.5.SP1">
      <Visible>False</Visible>
      <ProductName>.NET Framework 3.5 SP1</ProductName>
      <Install>false</Install>
    </BootstrapperPackage>
  </ItemGroup>
  <Import Project="$(MSBuildBinPath)\Microsoft.CSharp.targets" />
  <!-- To modify your build process, add your task inside one of the targets below and uncomment it. 
       Other similar extension points exist, see Microsoft.Common.targets.
  <Target Name="BeforeBuild">
  </Target>
  <Target Name="AfterBuild">
  </Target>
  -->
</Project>