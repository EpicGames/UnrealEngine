// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Windows.Forms;

namespace UnrealControls
{
	/// <summary>
	/// Custom renderer for drawing ToolStripItems in Unreal .NET tools. 
	/// Prevents highlighting of ToolStripItems when the application is not focused. 
	/// </summary>
	public class UnrealToolStripItemRenderer : ToolStripProfessionalRenderer
	{
		/// <summary>
		/// Determines if any of the running forms has focus.
		/// </summary>
		/// <returns>true if one form has focus; false, otherwise</returns>
		protected bool AppHasFocus()
		{
			bool bHasFocus = false;
			Form ActiveForm = System.Windows.Forms.Form.ActiveForm;

			// If the active form is null, then this app doesn't have focus.
			if (ActiveForm != null)
			{
				bHasFocus = ActiveForm.ContainsFocus;
			}

			return bHasFocus;
		}

		/// <summary>
		/// Disables menu item highlighting while mousing over 
		/// a menu item when the app doesn't have focus. 
		/// </summary>
		/// <param name="Args">The event holding the menu item to render.</param>
		protected override void OnRenderMenuItemBackground(ToolStripItemRenderEventArgs Args)
		{
			if ( AppHasFocus() )
			{
				base.OnRenderMenuItemBackground(Args);
			}
		}

		/// <summary>
		/// Disables tool strip item highlighting while mousing over 
		/// a tool strip item when the app doesn't have focus. 
		/// </summary>
		/// <param name="Args">The event holding the tool strip item to render.</param>
		protected override void OnRenderItemBackground(ToolStripItemRenderEventArgs Args)
		{
			if ( AppHasFocus() )
			{
				base.OnRenderItemBackground(Args);
			}
		}

		/// <summary>
		/// Disables button highlighting while mousing over 
		/// a button when the app doesn't have focus. 
		/// </summary>
		/// <param name="Args">The event holding the button to render.</param>
		protected override void OnRenderButtonBackground(ToolStripItemRenderEventArgs Args)
		{
			if ( AppHasFocus() )
			{
				base.OnRenderButtonBackground(Args);
			}
		}

		/// <summary>
		/// Disables drop down button highlighting while mousing over 
		/// a drop down button when the app doesn't have focus. 
		/// </summary>
		/// <param name="Args">The event holding the drop down button to render.</param>
		protected override void OnRenderDropDownButtonBackground(ToolStripItemRenderEventArgs Args)
		{
			if ( AppHasFocus() )
			{
				base.OnRenderDropDownButtonBackground(Args);
			}
		}

		/// <summary>
		/// Disables overflow button highlighting while mousing over 
		/// a overflow button when the app doesn't have focus. 
		/// </summary>
		/// <param name="Args">The event holding the overflow button to render.</param>
		protected override void OnRenderOverflowButtonBackground(ToolStripItemRenderEventArgs Args)
		{
			if ( AppHasFocus() )
			{
				base.OnRenderOverflowButtonBackground(Args);
			}
		}

		/// <summary>
		/// Disables split button highlighting while mousing over 
		/// a split button when the app doesn't have focus. 
		/// </summary>
		/// <param name="Args">The event holding the split button to render.</param>
		protected override void OnRenderSplitButtonBackground(ToolStripItemRenderEventArgs Args)
		{
			if ( AppHasFocus() )
			{
				base.OnRenderSplitButtonBackground(Args);
			}
			else
			{
				ToolStripSplitButton SplitButton = Args.Item as ToolStripSplitButton;
				ToolStripArrowRenderEventArgs DrawArrowEvent = new ToolStripArrowRenderEventArgs(	Args.Graphics,
																									SplitButton,
																									SplitButton.DropDownButtonBounds,
																									SplitButton.BackColor, 
																									ArrowDirection.Down);

				// If we don't do draw the down arrow while the form doesn't have focus, the arrow will 
				// disappear as the user hovers over the split button while the app is not focused.
				base.DrawArrow(DrawArrowEvent);
			}
		}
	}
}
