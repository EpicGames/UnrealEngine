// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace EpicCommonUtilities
{
    /// <summary>
    /// The kinds of statistics that the StatisticalFloat class tracks (or can optionally track, in the case of Median)
    /// </summary>
    public enum EStatType
    {
        Minimum,
        Maximum,
        Average,
        StDev,
        Median,
        Count
    };

    /// <summary>
    /// Keeps a running set of first-order statistics for a single variable
    /// </summary>
    public class StatisticalFloat
    {
        /// <summary>
        /// Should newly constructed StatisticalFloat instances track the median value?
        /// This requires O(N) storage instead of O(1) storage and it is only checked on construction.
        /// </summary>
        public static bool bSupportMedian = true;

        public double GetByType(EStatType Type)
        {
            switch (Type)
            {
                case EStatType.Minimum:
                    return Min;
                case EStatType.Maximum:
                    return Max;
                case EStatType.Average:
                    return Average;
                case EStatType.StDev:
                    return StandardDeviation;
                case EStatType.Median:
                    return Median;
                case EStatType.Count:
                    return MyCount;
                default:
                    return 0.0;
            }
        }

        public double Average
        {
            get { return MySum / MyCount; }
        }

        public double Min
        {
            get { return MyMin; }
        }

        public double Max
        {
            get { return MyMax; }
        }

        public double Value
        {
            get { return Average; }
            set { MyCount = 0; AddSample(value); }
        }

        /// <summary>
        /// Returns the median if bSupportMedian was true at construction and NaN otherwise
        /// </summary>
        public double Median
        {
            get
            {
                if (MySamples != null)
                {
                    MySamples.Sort();
                    if (MySamples.Count > 0)
                    {
                        return MySamples[MySamples.Count / 2];
                    }
                }
                return Math.Sqrt(-1.0);
            }
        }

        public double StandardDeviation
        {
            get
            {
                double numerator = (MyCount * MySumOfSquares - MySum * MySum);
                double denominator = MyCount * (MyCount - 1);

                return (MyCount > 1) ? Math.Sqrt(numerator / denominator) : 0.0;
            }
        }

        public double AggregateSum
        {
            get { return MySum; }
        }

        public StatisticalFloat()
        {
            if (bSupportMedian)
            {
                MySamples = new List<double>();
            }
        }

        public StatisticalFloat Clone()
        {
            StatisticalFloat Result = new StatisticalFloat();
            Result.AddSamples(this);
            return Result;
        }

        public StatisticalFloat(double InValue)
        {
            if (bSupportMedian)
            {
                MySamples = new List<double>();
            }

            MyCount = 0;
            AddSample(InValue);
        }

        public void AddSample(double InValue)
        {
            if (MyCount == 0)
            {
                MySum = InValue;
                MySumOfSquares = InValue * InValue;
                MyMin = InValue;
                MyMax = InValue;
            }
            else
            {
                MySum += InValue;
                MySumOfSquares += InValue * InValue;
                MyMin = Math.Min(MyMin, InValue);
                MyMax = Math.Max(MyMax, InValue);
            }

            if (MySamples != null)
            {
                MySamples.Add(InValue);
            }

            MyCount++;
        }

        public void AddSamples(StatisticalFloat Samples)
        {
            if (MyCount == 0)
            {
                MySum = Samples.MySum;
                MySumOfSquares = Samples.MySumOfSquares;
                MyMin = Samples.MyMin;
                MyMax = Samples.MyMax;
                MyCount = Samples.MyCount;
            }
            else
            {
                MySum += Samples.MySum;
                MySumOfSquares += Samples.MySumOfSquares;
                MyMin = Math.Min(MyMin, Samples.MyMin);
                MyMax = Math.Max(MyMax, Samples.MyMax);
                MyCount += Samples.MyCount;
            }

            if (MySamples != null)
            {
                MySamples.AddRange(Samples.MySamples);
            }
        }

        public void AddDistribution(StatisticalFloat Samples)
        {
            if (MyCount == 0)
            {
                AddSamples(Samples);
            }
            else if (Samples.MyCount > 0)
            {
                MyMin = MyMin + Samples.MyMin;
                MyMax = MyMax + Samples.MyMax;
                MySum = Average + Samples.Average;
                MySumOfSquares = MySum * MySum;
                MyCount = 1;
                
                if (MySamples != null)
                {
                    MySamples.Clear();
                    MySamples.Add(MySum);
                }
            }
        }

        public void ScaleBy(double ScaleFactor)
        {
            MySum *= ScaleFactor;
            MySumOfSquares *= Math.Abs(ScaleFactor);
            MyMin *= ScaleFactor;
            MyMax *= ScaleFactor;

            if (MySamples != null)
            {
                for (int i = 0; i < MySamples.Count; ++i)
                {
                    MySamples[i] = MySamples[i] * ScaleFactor;
                }
            }
        }
        
        public string ToStringSizesInKB()
        {
            return String.Format("avg={0:N1} KB [{1:N1}..{2:N1}]", Average/1024.0, MyMin/1024.0, MyMax/1024.0);
        }

        public string Min_ToStringInt()
        {
            return ((long)MyMin).ToString();
        }

        public string Max_ToStringInt()
        {
            return ((long)MyMax).ToString();
        }

        double MySumOfSquares = 0.0;
        double MySum = 0.0;
        double MyMin = 0.0;
        double MyMax = 0.0;
        long MyCount = 0;

        // Simple wasteful implementation; Tool is for offline anaysis and N is only on the order of hundreds typically
        // but the median tracking can be disabled by setting bSupportMedian to false before construction.
        List<double> MySamples = null;
    }
}