// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Drawing;
using System.Data;
using System.Text;
using System.Windows.Forms;
using System.Drawing.Drawing2D;
using System.Drawing.Imaging;

namespace UnrealControls
{
    //public delegate void XButtonClickedDelegate(object sender, EventArgs e);

    /// <summary>
    /// The X button on a VS.NET style tab control.
    /// </summary>
    public partial class XButton : UserControl
    {
		ProfessionalColorTable mColorTable = new ProfessionalColorTable();
		Bitmap mXBitmap;
		Color mBitmapDisplayColor = Color.Black;
        bool mMouseOver = false;

        /// <summary>
        /// Constructor.
        /// </summary>
        public XButton()
        {
            InitializeComponent();

			Bitmap template = Properties.Resources.XButton;
			mXBitmap = template.Clone(new Rectangle(0, 0, template.Width, template.Height), template.PixelFormat);
			mXBitmap.MakeTransparent(Color.White);
        }

        /// <summary>
		/// Updates the colors in the bitmap to show the correct text color.
		/// </summary>
		unsafe void UpdateBitmapDisplayColor()
		{
			if(mBitmapDisplayColor.ToArgb() != SystemColors.ControlText.ToArgb())
			{
				BitmapData data = mXBitmap.LockBits(new Rectangle(0, 0, mXBitmap.Width, mXBitmap.Height), System.Drawing.Imaging.ImageLockMode.ReadWrite, System.Drawing.Imaging.PixelFormat.Format32bppArgb);

				int* clr = (int*)data.Scan0.ToPointer();
				int numPixels = mXBitmap.Height * mXBitmap.Width;

				for(int i = 0; i < numPixels; ++i, ++clr)
				{
					if(*clr == mBitmapDisplayColor.ToArgb())
					{
						*clr = SystemColors.ControlText.ToArgb();
					}
				}

				mXBitmap.UnlockBits(data);

				mBitmapDisplayColor = SystemColors.ControlText;
			}
		}

        /// <summary>
        /// Called to draw the control.
        /// </summary>
        /// <param name="e">Event args.</param>
        protected override void OnPaint(PaintEventArgs e)
        {
			UpdateBitmapDisplayColor();

			if(mMouseOver)
			{
				Rectangle rect = new Rectangle(0, 0, this.Width - 1, this.Height - 1);
				using(LinearGradientBrush bgBrush = new LinearGradientBrush(rect, mColorTable.MenuItemSelectedGradientBegin, mColorTable.MenuItemSelectedGradientEnd, LinearGradientMode.Vertical))
				{
					e.Graphics.FillRectangle(bgBrush, rect);
				}

				using(Pen pen = new Pen(mColorTable.MenuItemBorder))
				{
					e.Graphics.DrawRectangle(pen, rect);
				}
			}

			e.Graphics.DrawImage(mXBitmap, this.ClientRectangle);

            base.OnPaint(e);
        }

        /// <summary>
        /// Called when the mouse enters the control.
        /// </summary>
        /// <param name="e">Event args.</param>
        protected override void OnMouseEnter(EventArgs e)
        {
            mMouseOver = true;
            base.OnMouseEnter(e);

            Invalidate();
        }

        /// <summary>
        /// Called when the mouse leaves the control.
        /// </summary>
        /// <param name="e">Event args.</param>
        protected override void OnMouseLeave(EventArgs e)
        {
            mMouseOver = false;
            base.OnMouseLeave(e);

            Invalidate();
        }
    }
}
